// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
	!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#pragma once


#ifndef __cplusplus
// Change this to force recompilation of all shaders refeencing this file.
#pragma message("UESHADERMETADATA_VERSION 8F92177A-03B9-4AA9-A983-3CA04895E84A")
#endif


#define ENVCOMP_FLAG_SKYATMOSPHERE_HOLDOUT			0x01
#define ENVCOMP_FLAG_VOLUMETRICCLOUD_HOLDOUT		0x02
#define ENVCOMP_FLAG_EXPONENTIALFOG_HOLDOUT			0x04

#define ENVCOMP_FLAG_SKYATMOSPHERE_RENDERINMAIN		0x08
#define ENVCOMP_FLAG_VOLUMETRICCLOUD_RENDERINMAIN	0x10
#define ENVCOMP_FLAG_EXPONENTIALFOG_RENDERINMAIN	0x20


// Using macro functions to avoid having to move the definition in cpp to avoid the multiple definition in different obj files.
#define IsSkyAtmosphereHoldout(EnvironmentComponentsFlags)   ((EnvironmentComponentsFlags[0] & ENVCOMP_FLAG_SKYATMOSPHERE_HOLDOUT)   > 0)
#define IsVolumetricCloudHoldout(EnvironmentComponentsFlags) ((EnvironmentComponentsFlags[0] & ENVCOMP_FLAG_VOLUMETRICCLOUD_HOLDOUT) > 0)
#define IsExponentialFogHoldout(EnvironmentComponentsFlags)  ((EnvironmentComponentsFlags[0] & ENVCOMP_FLAG_EXPONENTIALFOG_HOLDOUT)  > 0)

#define IsSkyAtmosphereRenderedInMain(EnvironmentComponentsFlags)   ((EnvironmentComponentsFlags[0] & ENVCOMP_FLAG_SKYATMOSPHERE_RENDERINMAIN)   > 0)
#define IsVolumetricCloudRenderedInMain(EnvironmentComponentsFlags) ((EnvironmentComponentsFlags[0] & ENVCOMP_FLAG_VOLUMETRICCLOUD_RENDERINMAIN) > 0)
#define IsExponentialFogRenderedInMain(EnvironmentComponentsFlags)  ((EnvironmentComponentsFlags[0] & ENVCOMP_FLAG_EXPONENTIALFOG_RENDERINMAIN)  > 0)

