// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
VirtualShadowMapDefinitions.h: used in virtual shadow map shaders and C++ code to define common constants
!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#pragma once

#define FROXEL_TILE_SIZE (8)
#define FROXEL_INVALID_SLICE (1 << 28)
#define FROXEL_INVALID_PACKED_SLICE (FROXEL_INVALID_SLICE << 2)
#define FROXEL_PACKED_SLICE_BIAS (1 << 20) // used to represent negative slices (TODO: dont)
#define FROXEL_INDIRECT_ARG_WORKGROUP_SIZE (64)


#ifdef __cplusplus
#include "HLSLTypeAliases.h"

namespace UE::HLSL
{
#endif

struct FPackedFroxel
{
	uint XY;
	int Z;
};


#ifdef __cplusplus
} // namespace UE::HLSL

namespace Froxel
{
	using FPackedFroxel = UE::HLSL::FPackedFroxel;
}

#endif