// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*================================================================================================
	HLSLMathAliases.h: include this file in headers that are used in both C++ and HLSL
	to define aliases for common HSLS math functions to C++ UE math functions.
=================================================================================================*/

#include "Math/UnrealMathUtility.h"

namespace UE::HLSL
{
	inline float log2(float Value)
	{
		return FMath::Log2(Value);
	}

	inline float floor(float Value)
	{
		return FMath::Floor(Value);
	}

	template <typename ValueType>
	inline float max(ValueType A, ValueType B)
	{
		return FMath::Max(A, B);
	}
	template <typename ValueType>
	inline float min(ValueType A, ValueType B)
	{
		return FMath::Min(A, B);
	}
}

