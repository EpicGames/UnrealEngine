// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(__cplusplus)
	#define INCLUDED_FROM_CPP_CODE  1
	#define INCLUDED_FROM_HLSL_CODE 0
#else
	#define INCLUDED_FROM_CPP_CODE  0
	#define INCLUDED_FROM_HLSL_CODE 1
#endif

#if INCLUDED_FROM_HLSL_CODE
#define UE_REG_CONCATENATE2(a, b) a##b
#define UE_REG_CONCATENATE(a, b) UE_REG_CONCATENATE2(a, b)
#define UE_HLSL_REGISTER(InType, InIndex, InSpace) register(UE_REG_CONCATENATE(InType, InIndex), UE_REG_CONCATENATE(space, InSpace))
#endif

// Default binding space
#define UE_HLSL_SPACE_DEFAULT 0

// Default register space for hit group (closest hit, any hit, intersection) shader resources
#define UE_HLSL_SPACE_RAY_TRACING_LOCAL 0

// Register space for ray generation and miss shaders
#define UE_HLSL_SPACE_RAY_TRACING_GLOBAL 1

// Register space for "system" parameters (index buffer, vertex buffer, fetch parameters)
#define UE_HLSL_SPACE_RAY_TRACING_SYSTEM 2

// Register space for for resources referenced by a work graph pixel shader local root signature
// Needs to match the default binding space since we don't specialize work graph pixel shader compilation
#define UE_HLSL_SPACE_WORK_GRAPH_LOCAL_PIXEL UE_HLSL_SPACE_DEFAULT

// Register space for for resources referenced by a global work graph root signature
#define UE_HLSL_SPACE_WORK_GRAPH_GLOBAL 1

// Register space for resources referenced by a work graph compute or mesh shader local root signature
#define UE_HLSL_SPACE_WORK_GRAPH_LOCAL 2

// Register space for shader root constants (4xu32)
#define UE_HLSL_SPACE_SHADER_ROOT_CONSTANTS 3

// Register space for static shader bindings
#define UE_HLSL_SPACE_STATIC_SHADER_BINDINGS 4

// Register space for UE diagnotic debug buffer
#define UE_HLSL_SPACE_DIAGNOSTIC 999

// NOTE: AMD AGS uses AGS_DX12_SHADER_INSTRINSICS_SPACE_ID=2147420894

// Nvidia Shader Extensions
#define UE_HLSL_SLOT_NV_SHADER_EXTN 0
#define UE_HLSL_SPACE_NV_SHADER_EXTN 1001

#define NV_SHADER_EXTN_SLOT UE_REG_CONCATENATE(u, UE_HLSL_SLOT_NV_SHADER_EXTN)
#define NV_SHADER_EXTN_REGISTER_SPACE UE_REG_CONCATENATE(space, UE_HLSL_SPACE_NV_SHADER_EXTN)
