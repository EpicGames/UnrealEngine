// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#if defined(HLSL_STATIC_ASSERT) // if this macro has not been defined by a platform specific header already ...

HLSL_STATIC_ASSERT(true, "Verify that custom static assert works");

#else

#if defined(__cplusplus) || COMPILER_PSSL

// Included from C++ code, use C++11 keyword
#define HLSL_STATIC_ASSERT(Expr, Msg)	static_assert(Expr, Msg)

#elif COMPILER_DXC && !COMPILER_VULKAN

// DXC only supports C11 style static asserts
#define HLSL_STATIC_ASSERT(Expr, Msg) _Static_assert(Expr, Msg)

#else

// this compiler does not support static_assert, just ignore the statements
#define HLSL_STATIC_ASSERT(Expr, Msg)  

#endif

#endif
