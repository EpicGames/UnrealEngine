// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*================================================================================================
	HLSLTypeAliases.h: include this file in headers that are used in both C++ and HLSL
	to define type aliases that map HLSL types to C++ UE types.
=================================================================================================*/

#include "Math/MathFwd.h"
#include "Misc/LargeWorldRenderPosition.h"

namespace UE::HLSL
{
	using int2 = FIntVector2;
	using int3 = FIntVector3;
	using int4 = FIntVector4;

	using uint = uint32;
	using uint2 = FUintVector2;
	using uint3 = FUintVector3;
	using uint4 = FUintVector4;

	//dword
	//half

	using float2 = FVector2f;
	using float3 = FVector3f;
	using float4 = FVector4f;

	using float4x4 = FMatrix44f;

	using FLWCVector3 = TLargeWorldRenderPosition<float>;
}

