// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
	HairStrandsDefinitions.ush: used in ray tracing shaders and C++ code to define common constants
	!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#pragma once

// Change this to force recompilation of all Substrate dependent shaders (use https://www.random.org/cgi-bin/randbyte?nbytes=4&format=h)
#define HAIRSTRANDS_SHADER_VERSION 0x3ba5af0e

// Curve Attribute index
#define HAIR_CURVE_ATTRIBUTE_ROOTUV 0
#define HAIR_CURVE_ATTRIBUTE_SEED 1
#define HAIR_CURVE_ATTRIBUTE_LENGTH 2
#define HAIR_CURVE_ATTRIBUTE_CLUMPID 3
#define HAIR_CURVE_ATTRIBUTE_CLUMPID3 4
#define HAIR_CURVE_ATTRIBUTE_COUNT 5

// Point Attribute index
#define HAIR_POINT_ATTRIBUTE_COLOR 0
#define HAIR_POINT_ATTRIBUTE_ROUGHNESS 1
#define HAIR_POINT_ATTRIBUTE_AO 2
#define HAIR_POINT_ATTRIBUTE_COUNT 3

// Groom limits (based on encoding)
#define HAIR_MAX_NUM_POINT_PER_CURVE ((1u<<8)-1u)
#define HAIR_MAX_NUM_POINT_PER_GROUP ((1u<<24)-1u)
#define HAIR_MAX_NUM_CURVE_PER_GROUP ((1u<<22)-1u)

#define HAIR_ATTRIBUTE_INVALID_OFFSET 0xFFFFFFFF

// Hair interpolation (in bytes)
#define HAIR_INTERPOLATION_CURVE_STRIDE 8
#define HAIR_INTERPOLATION_POINT_STRIDE 2
#define HAIR_INTERPOLATION_MAX_GUIDE_COUNT 2

#define HAIR_INTERPOLATION_CARDS_GUIDE_STRIDE 4

// Max number of discrete LOD that a hair group can have
#define MAX_HAIR_LOD 8

// Max split for raytracing geometry
#define STRANDS_PROCEDURAL_INTERSECTOR_MAX_SPLITS 4

// Number of vertex per control-point
#define HAIR_POINT_TO_VERTEX_FOR_TRISTRP 2u
#define HAIR_POINT_TO_VERTEX_FOR_TRILIST 6u

#ifndef __cplusplus
#if USE_HAIR_TRIANGLE_STRIP
#define HAIR_POINT_TO_VERTEX HAIR_POINT_TO_VERTEX_FOR_TRISTRP
#else
#define HAIR_POINT_TO_VERTEX HAIR_POINT_TO_VERTEX_FOR_TRILIST
#endif
#endif

// Number of triangle per control-point
#define HAIR_POINT_TO_TRIANGLE 2u

// Type of control points
#define HAIR_CONTROLPOINT_INSIDE 0u
#define HAIR_CONTROLPOINT_START	 1u
#define HAIR_CONTROLPOINT_END	 2u

// Group size for dispatching based on a groom vertex/curve/cluster count. 
// This defines ensures the group size is consistent across the hair pipeline, 
// and ensures dispatch count are smaller than 65k limits
#define HAIR_VERTEXCOUNT_GROUP_SIZE  1024u
#define HAIR_CURVECOUNT_GROUP_SIZE   1024u
#define HAIR_CLUSTERCOUNT_GROUP_SIZE 1024u

#define MAX_HAIR_MACROGROUP_COUNT 16

// HAIR_XXX_ATTRIBUTE_MAX rounded to 4
#define HAIR_CURVE_ATTRIBUTE_OFFSET_COUNT ((HAIR_CURVE_ATTRIBUTE_COUNT + 3) / 4)
#define HAIR_POINT_ATTRIBUTE_OFFSET_COUNT ((HAIR_POINT_ATTRIBUTE_COUNT + 3) / 4)

// Pack all offset into uint4
#define PACK_HAIR_ATTRIBUTE_OFFSETS(Out, In, Offset, Count) \
	for (uint32 AttributeIt4 = 0; AttributeIt4 < Offset; ++AttributeIt4) \
	{ \
		Out[AttributeIt4] = FUintVector4(HAIR_ATTRIBUTE_INVALID_OFFSET, HAIR_ATTRIBUTE_INVALID_OFFSET, HAIR_ATTRIBUTE_INVALID_OFFSET, HAIR_ATTRIBUTE_INVALID_OFFSET); \
	} \
	for (uint32 AttributeIt = 0; AttributeIt < Count; ++AttributeIt) \
	{ \
		const uint32 Index4 = AttributeIt & (~0x3); \
		const uint32 SubIndex = AttributeIt - Index4; \
		const uint32 IndexDiv4 = Index4 >> 2u; \
		Out[IndexDiv4][SubIndex] = In[AttributeIt]; \
	}

#define PACK_CURVE_HAIR_ATTRIBUTE_OFFSETS(Out, In) PACK_HAIR_ATTRIBUTE_OFFSETS(Out, In, HAIR_CURVE_ATTRIBUTE_OFFSET_COUNT, HAIR_CURVE_ATTRIBUTE_COUNT)
#define PACK_POINT_HAIR_ATTRIBUTE_OFFSETS(Out, In) PACK_HAIR_ATTRIBUTE_OFFSETS(Out, In, HAIR_POINT_ATTRIBUTE_OFFSET_COUNT, HAIR_POINT_ATTRIBUTE_COUNT)

// Data for LODing points of a curve
// The number of bits per vertex describe the maxium LOD decomposition
// Each curve's point encode its minium LOD at which it becomes active
#define HAIR_POINT_LOD_BIT_COUNT 4
#define HAIR_POINT_LOD_COUNT_PER_UINT (32u/4u)
#define HAIR_POINT_LOD_COUNT_PER_UINT_DIV_AS_SHIFT 3

#define HAIR_RBF_ENTRY_COUNT(InSampleCount)	((InSampleCount)+4u)

#define HAIR_CARDS_MAX_TEXTURE_COUNT 6

#define HAIR_MIN_VOXEL_SIZE 0.01f
#define HAIR_MAX_VOXEL_SIZE 100.f

// Hair instance flags 
#define HAIR_FLAGS_SCATTER_SCENE_LIGHT 0x1u
#define HAIR_FLAGS_STABLE_RASTER 0x2u
#define HAIR_FLAGS_RAYTRACING_GEOMETRY 0x4u
#define HAIR_FLAGS_HOLDOUT 0x8u
#define HAIR_FLAGS_CAST_CONTACT_SHADOW 0x10u

#ifndef __cplusplus //HLSL
bool HasHairFlags(uint In, uint Flags)
#else
FORCEINLINE bool HasHairFlags(uint32 In, uint32 Flags)
#endif
{
	return (In & Flags) != 0;
}

///////////////////////////////////////////////////////////////////////////////////////////////////
// Types

// Hair control point layout
// [                  uint2                   ]
// [      x     ][              y             ]
// [   32bits   ][           32bits           ]
// [  16 ][  16 ][ 16  ][   8  ][   6  ][  2  ]
// [ 0-15][16-32][ 0-15][ 16-23][ 24-29][30-31]
// [Pos.x][Pos.y][Pos.z][CoordU][Radius][ Type]
#define FPackedHairPosition uint2
#define FPackedHairPositionStrideInBytes 8u

// Compressed Hair control point layout - Packed 4 points per 16-bytes
// [                            uint4                                 ]
// [Position0+Type][Position1+Type][Position2+Type][ Radius012|CoordU ]
// [   32bits     ][   32bits     ][   32bits     ][      32bits      ]
// [10][10][10][ 2][10][10][10][ 2][10][10][10][ 2][ 6][ 6][ 6][ 7][ 7]
#define FCompressedHairPositions uint4
#define FCompressedHairPositionsStrideInBytes 16u
#define HAIR_POINT_COUNT_PER_COMPRESSED_POSITION_CHUNK 3