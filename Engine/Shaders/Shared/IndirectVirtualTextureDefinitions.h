// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef __cplusplus
#include "HLSLTypeAliases.h"
namespace UE::HLSL
{
#endif  // __cplusplus

struct FIndirectVirtualTextureUniform
{
	uint3 Pad;
	uint  UniformCountSub1;
	uint4 PackedPageTableUniform[2];
	uint4 PackedUniform;
};

struct FIndirectVirtualTextureEntry
{
	// Could be packed in one uint if we accept 4k textures as max
	uint2 PackedCoordinateAndSize;
};

static const uint IndirectVirtualTextureUniformDWord4Count = 4u;

#ifdef __cplusplus
} // namespace
#endif // __cplusplus
