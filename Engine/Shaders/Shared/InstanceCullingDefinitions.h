// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define INSTANCE_CULLING_PRESERVE_INSTANCE_ORDER_BIT_MASK 1U
#define INSTANCE_CULLING_DYNAMIC_INSTANCE_DATA_OFFSET_BIT_MASK 2U

#define INSTANCE_CULLING_PAYLOAD_NUM_COMMON_BITS 2U

#define INSTANCE_CULLING_FLAGS_BITS (2u)
#define INSTANCE_CULLING_FLAGS_ALLOW_OCCLUSION_CULLING	(1u << 0u)
#define INSTANCE_CULLING_FLAGS_DRAW_ONLY_SELECTED		(1u << 1u)