// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define LIGHT_TYPE_DIRECTIONAL		0 
#define LIGHT_TYPE_POINT			1 
#define LIGHT_TYPE_SPOT				2 
#define LIGHT_TYPE_RECT				3 
#define LIGHT_TYPE_MAX				4 

#define LIGHT_EXTRA_DATA_BIT_OFFSET_SHADOW_MAP_CHANNEL_MASK 0               // 8 bits
#define LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHTING_CHANNEL_MASK 8                 // 3 bits
#define LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHT_TYPE 11                           // LightType_NumBits (2) bits
#define LIGHT_EXTRA_DATA_BIT_OFFSET_CAST_SHADOW 13                          // 1 bit
#define LIGHT_EXTRA_DATA_BIT_OFFSET_HAS_LIGHT_FUNCTION 14                   // 1 bit
#define LIGHT_EXTRA_DATA_BIT_OFFSET_LIGHT_FUNCTION_ATLAS_LIGHT_INDEX 15     // 8 bits
#define LIGHT_EXTRA_DATA_BIT_OFFSET_AFFECT_TRANSLUCENT_LIGHTING 23          // 1 bit
#define LIGHT_EXTRA_DATA_BIT_OFFSET_MEGA_LIGHT 24                           // 1 bit
#define LIGHT_EXTRA_DATA_BIT_OFFSET_CLUSTERED_DEFERRED_SUPPORTED 25         // 1 bit
#define LIGHT_EXTRA_DATA_BIT_OFFSET_CAST_VOLUMETRIC_SHADOWS 26              // 1 bit
#define LIGHT_EXTRA_DATA_BIT_OFFSET_USE_TRANSMISSION 27                     // 1 bit

// 28 bits used, 4 bits free
