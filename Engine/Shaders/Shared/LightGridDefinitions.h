// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
	LightGridDefinitions.h: used in light grid shaders and C++ code to define common constants
=================================================================================================*/

#pragma once

// Must updating packing in FCellWriter when modifying these
#define LIGHT_GRID_CELL_WRITER_MAX_NUM_PRIMITIVES (1 << 24) // must be power to two
#define LIGHT_GRID_CELL_WRITER_MAX_PRIMITIVE_INDEX (LIGHT_GRID_CELL_WRITER_MAX_NUM_PRIMITIVES - 1)

#define LIGHT_GRID_CELL_WRITER_MAX_NUM_LINKS (1 << 24) // must be power to two
#define LIGHT_GRID_CELL_WRITER_MAX_LINK_OFFSET (LIGHT_GRID_CELL_WRITER_MAX_NUM_LINKS - 1)
