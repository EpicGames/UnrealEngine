// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef __cplusplus
#include "HLSLTypeAliases.h"

namespace UE::HLSL
{
#endif
#ifndef __cplusplus //HLSL
#include "/Engine/Private/LargeWorldCoordinates.ush"
#endif


/**
 * Has a 1:1 mapping with FLightRenderParameters, but unlike FLightShaderParameters, this is view-independent
 */
struct FLightSceneData
{
	// Position of the light in world space.
	FDFVector3 WorldPosition;

	// 1 / light's falloff radius from Position.
	float InvRadius;

	// The exponent for the falloff of the light intensity from the distance.
	float FalloffExponent;

	// Direction of the light if applies.
	float3 Direction;

	float InverseExposureBlend;

	// One tangent of the light if applies.
	// Note: BiTangent is on purpose not stored for memory optimisation purposes.
	float3 Tangent;

	// Radius of the point light.
	float SourceRadius;

	// Dimensions of the light, for spot light, but also
	float2 SpotAngles;

	// Radius of the soft source.
	float SoftSourceRadius;

	// Other dimensions of the light source for rect light specifically.
	float SourceLength;

	// Barn door angle for rect light
	float RectLightBarnCosAngle;

	// Barn door length for rect light
	float RectLightBarnLength;

	// Factor to applies on the specular.
	float SpecularScale;

	// Factor to applies on the diffuse.
	float DiffuseScale;
};

// Alternative for sizeof(FLightSceneData). FXC reserves keyword 'sizeof' so we cannot use it for shader permutations that target PCD3D_SM5.
//  struct struct.FLightSceneData
//  {
//
//      struct struct.FDFVector3
//      {
//
//          float3 a;                                 ; Offset:    0
//          float3 b;                                 ; Offset:   12
//      
//      } WorldPosition;                              ; Offset:    0
//
//      float InvRadius;                              ; Offset:   24
//      float FalloffExponent;                        ; Offset:   28
//      float3 Direction;                             ; Offset:   32
//      float InverseExposureBlend;                   ; Offset:   44
//      float3 Tangent;                               ; Offset:   48
//      float SourceRadius;                           ; Offset:   60
//      float2 SpotAngles;                            ; Offset:   64
//      float SoftSourceRadius;                       ; Offset:   72
//      float SourceLength;                           ; Offset:   76
//      float RectLightBarnCosAngle;                  ; Offset:   80
//      float RectLightBarnLength;                    ; Offset:   84
//      float SpecularScale;                          ; Offset:   88
//      float DiffuseScale;                           ; Offset:   92
//  
//  } $Element;                                       ; Offset:    0 Size:    96
#define SIZEOF_LIGHT_SCENE_DATA 96

#ifdef __cplusplus
} // namespace
using FLightSceneData = UE::HLSL::FLightSceneData;
#endif
