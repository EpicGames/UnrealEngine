// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef __cplusplus
#include "HLSLTypeAliases.h"

namespace UE::HLSL
{
#endif

/**
 * Light view-dependent data
 */
struct FLightViewData
{
	// TODO: pack members to reduce memory overhead/bandwidth
	float3 TranslatedWorldPosition;
	float3 Color;
	float VolumetricScatteringIntensity;
	uint VirtualShadowMapId;

	uint LightSceneInfoExtraDataPacked;
	
	// Rect. light atlas transformation
	float2 RectLightAtlasUVOffset;
	float2 RectLightAtlasUVScale;
	float RectLightAtlasMaxLevel;

	// could pack IESAtlasIndex with other data in the future since it doesn't require 32 bits
	// FLocalLightData packs it in 16 bits (see UnpackLigthIESAtlasIndex(...))
	// could probably go down to 8 bits (with some logic in GIESTextureManager to warn about overflow)
	float IESAtlasIndex;
};

#ifdef __cplusplus
} // namespace
using FLightViewData = UE::HLSL::FLightViewData;
#endif
