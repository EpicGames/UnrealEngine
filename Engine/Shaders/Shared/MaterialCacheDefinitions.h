// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef __cplusplus
#include "HLSLTypeAliases.h"

namespace UE::HLSL
{
#endif

struct FMaterialCacheTagEntry
{
	uint2 PackedUniform;
};

struct FMaterialCacheBinData
{
	// DW0
	uint3 ABufferPhysicalPosition;
	uint  PrimitiveData;
	// DW4
	float4 UVMinAndThreadAdvance;
	// DW8
	float4 UVMinAndInvSize;
	// DW12
	uint4 Pad16;
};

struct FMaterialCachePageWriteData
{
	// DW0
	uint3 ABufferPhysicalPosition;
	uint  Pad;
	// DW4
	uint2 VTPhysicalPosition;
	uint  Pad2;
};

#ifdef __cplusplus
enum EMaterialCacheFlag
{
	MatCache_None = 0u,
	MatCache_DefaultBottomLayer = 1u << 0u,
};
#else // __cplusplus
// HLSL Cross-Compiler Workaround
#define EMaterialCacheFlag uint
static uint MatCache_None = 0u;
static uint MatCache_DefaultBottomLayer = 1u << 0u;
#endif // __cplusplus

#ifdef __cplusplus
} // namespace UE::HLSL
#endif
