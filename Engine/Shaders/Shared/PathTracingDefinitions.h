// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Constants for 'SamplerType'
#define PATHTRACER_SAMPLER_DEFAULT			0
#define PATHTRACER_SAMPLER_ERROR_DIFFUSION	1

// Constants for 'PATH_TRACER_KERNEL_TYPE'
#define PATH_TRACER_KERNEL_CAMERA 0   // Camera rays only
#define PATH_TRACER_KERNEL_INNER  1   // Internal single-bounce
#define PATH_TRACER_KERNEL_TAIL   2   // Internal tail bounces (from a fixed point up to the max)

// Constants for the 'Flags' field of FPathTracingLight
#define PATHTRACER_FLAG_LIGHTING_CHANNEL_MASK				(7 << 0)	// Which lighting channel is this light assigned to?
#define PATHTRACER_FLAG_TRANSMISSION_MASK					(1 << 3)	// Does the light affect the transmission side?
#define PATHTRACER_FLAG_NON_INVERSE_SQUARE_FALLOFF_MASK		(1 << 4)	// Does the light have a non-inverse square decay?
#define PATHTRACER_FLAG_STATIONARY_MASK						(1 << 5)	// Only used by GPULightmass
#define PATHTRACER_FLAG_TYPE_MASK							(7 << 6)
#define PATHTRACING_LIGHT_SKY								(0 << 6)
#define PATHTRACING_LIGHT_DIRECTIONAL						(1 << 6)
#define PATHTRACING_LIGHT_POINT								(2 << 6)
#define PATHTRACING_LIGHT_SPOT								(3 << 6)
#define PATHTRACING_LIGHT_RECT								(4 << 6)
#define PATHTRACER_FLAG_CAST_SHADOW_MASK 					(1 << 9)
#define PATHTRACER_FLAG_CAST_VOL_SHADOW_MASK 				(1 << 10)
#define PATHTRACER_FLAG_CAST_CLOUD_SHADOW_MASK              (1 << 11)
#define PATHTRACER_FLAG_HAS_RECT_TEXTURE_MASK				(1 << 12)

#define PATHTRACER_MASK_CAMERA_OPAQUE                       0x01            // Visible to Camera and BlendMode=Opaque+Masked 
#define PATHTRACER_MASK_CAMERA_TRANSLUCENT                  0x02            // Visible to Camera and BlendMode=Translucent+others (all blend modes outside of opaque and masked)
#define PATHTRACER_MASK_SHADOW                              0x04            // Casts shadows: all blend modes
#define PATHTRACER_MASK_INDIRECT_OPAQUE                     0x08            // Affects Indirect and BlendMode=Opaque+Masked 
#define PATHTRACER_MASK_INDIRECT_TRANSLUCENT                0x10            // Affects Indirect and BlendMode=Translucent+others
#define PATHTRACER_MASK_REFLECTION_OPAQUE                   0x20            // Visible in Reflections and BlendMode=Opaque+Masked
#define PATHTRACER_MASK_REFLECTION_TRANSLUCENT              0x40			// Visible in Reflections and BlendMode=Translucent+others
#define PATHTRACER_MASK_SSS_RANDOM_WALK                     0x80            // Visible to SSS random walks

#define PATHTRACER_MASK_IGNORE                              0x00            // used when mapping general tracing mask to path tracing mask
#define PATHTRACER_MASK_ALL                                 0xFF

// Constants for light contribution types (AOV decomposition of the image)
// Leaving all constants enabled creates the beauty image, but turning off some bits allows
// the path tracer to create an image with only certain components enabled
#define PATHTRACER_CONTRIBUTION_EMISSIVE                     1
#define PATHTRACER_CONTRIBUTION_DIFFUSE                      2
#define PATHTRACER_CONTRIBUTION_SPECULAR                     4
#define PATHTRACER_CONTRIBUTION_VOLUME                       8

// Constants for the path tracer light grid
#define PATHTRACER_LIGHT_GRID_SINGULAR_MASK					0x80000000u
#define PATHTRACER_LIGHT_GRID_LIGHT_COUNT_MASK				0x7FFFFFFFu

// Constants related to volumetric support
#define VOLUMEID_ATMOSPHERE				0
#define VOLUMEID_CLOUDS					1
#define VOLUMEID_FOG					2
#define VOLUMEID_HETEROGENEOUS_VOLUMES	3
#define PATH_TRACER_MAX_VOLUMES			4

#define PATH_TRACER_VOLUME_ENABLE_BIT						(1u)
#define PATH_TRACER_VOLUME_ENABLE_ATMOSPHERE  				(PATH_TRACER_VOLUME_ENABLE_BIT << VOLUMEID_ATMOSPHERE)
#define PATH_TRACER_VOLUME_ENABLE_CLOUDS                    (PATH_TRACER_VOLUME_ENABLE_BIT << VOLUMEID_CLOUDS)
#define PATH_TRACER_VOLUME_ENABLE_FOG               		(PATH_TRACER_VOLUME_ENABLE_BIT << VOLUMEID_FOG)
#define PATH_TRACER_VOLUME_ENABLE_HETEROGENEOUS_VOLUMES 	(PATH_TRACER_VOLUME_ENABLE_BIT << VOLUMEID_HETEROGENEOUS_VOLUMES)
#define PATH_TRACER_VOLUME_ENABLE_MASK                      ((1u << PATH_TRACER_MAX_VOLUMES) - 1)

#define PATH_TRACER_VOLUME_HOLDOUT_BIT						(PATH_TRACER_VOLUME_ENABLE_BIT << PATH_TRACER_MAX_VOLUMES)
#define PATH_TRACER_VOLUME_HOLDOUT_ATMOSPHERE  				(PATH_TRACER_VOLUME_HOLDOUT_BIT << VOLUMEID_ATMOSPHERE)
#define PATH_TRACER_VOLUME_HOLDOUT_CLOUDS   				(PATH_TRACER_VOLUME_HOLDOUT_BIT << VOLUMEID_CLOUDS)
#define PATH_TRACER_VOLUME_HOLDOUT_FOG               		(PATH_TRACER_VOLUME_HOLDOUT_BIT << VOLUMEID_FOG)
#define PATH_TRACER_VOLUME_HOLDOUT_HETEROGENEOUS_VOLUMES	(PATH_TRACER_VOLUME_HOLDOUT_BIT << VOLUMEID_HETEROGENEOUS_VOLUMES)
#define PATH_TRACER_VOLUME_HOLDOUT_MASK                     (PATH_TRACER_VOLUME_ENABLE_MASK << PATH_TRACER_MAX_VOLUMES)

#define PATH_TRACER_VOLUME_USE_ANALYTIC_TRANSMITTANCE       (256u)

// These flags are reserved for passing information to the cloud callable shader
#define PATH_TRACER_VOLUME_CALLABLE_FLAGS_TRANSMITTANCE     (512u) // only compute transmittance for the cloud portion
#define PATH_TRACER_VOLUME_CALLABLE_FLAGS_GET_SAMPLE        (1024u) // perform RIS sampling for a scatter location (else do transmittance/emission/alpha calc only)
#define PATH_TRACER_VOLUME_CALLABLE_FLAGS_BOUNCE_MASK       (2048u * 31) // What bounce are we rendering? (5 bits, so bounces>31 are clamped)
#define PATH_TRACER_VOLUME_CALLABLE_FLAGS_BOUNCE_SHIFT      (11)

// Flags beyond this point are not visible to the cloud callable shader
#define PATH_TRACER_VOLUME_SHOW_PLANET_GROUND               (65536u)

// Constants related to debugging
#define PATH_TRACER_DEBUG_VIZ_RADIANCE					0 
#define PATH_TRACER_DEBUG_VIZ_WORLD_NORMAL				1 
#define PATH_TRACER_DEBUG_VIZ_WORLD_SMOOTH_NORMAL		2
#define PATH_TRACER_DEBUG_VIZ_WORLD_GEO_NORMAL			3
#define PATH_TRACER_DEBUG_VIZ_BASE_COLOR				4 
#define PATH_TRACER_DEBUG_VIZ_DIFFUSE_COLOR				5 
#define PATH_TRACER_DEBUG_VIZ_SPECULAR_COLOR			6 
#define PATH_TRACER_DEBUG_VIZ_TRANSPARENCY_COLOR		7
#define PATH_TRACER_DEBUG_VIZ_METALLIC					8 
#define PATH_TRACER_DEBUG_VIZ_SPECULAR					9 
#define PATH_TRACER_DEBUG_VIZ_ROUGHNESS					10 
#define PATH_TRACER_DEBUG_VIZ_IOR						11
#define PATH_TRACER_DEBUG_VIZ_SHADING_MODEL				12 
#define PATH_TRACER_DEBUG_VIZ_LIGHTING_CHANNEL_MASK		13  
#define PATH_TRACER_DEBUG_VIZ_CUSTOM_DATA0				14  
#define PATH_TRACER_DEBUG_VIZ_CUSTOM_DATA1				15
#define PATH_TRACER_DEBUG_VIZ_WORLD_POSITION			16  
#define PATH_TRACER_DEBUG_VIZ_PRIMARY_RAYS				17  
#define PATH_TRACER_DEBUG_VIZ_WORLD_TANGENT				18 
#define PATH_TRACER_DEBUG_VIZ_ANISOTROPY				19 
#define PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_COUNT          20
#define PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_AXIS           21
#define PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_LIST_HASH      22
#define PATH_TRACER_DEBUG_VIZ_LIGHT_GRID_SINGULAR       23
#define PATH_TRACER_DEBUG_VIZ_DECAL_GRID_COUNT 			24
#define PATH_TRACER_DEBUG_VIZ_DECAL_GRID_AXIS			25
#define PATH_TRACER_DEBUG_VIZ_DECAL_GRID_LIST_HASH      26
#define PATH_TRACER_DEBUG_VIZ_VOLUME_LIGHT_COUNT        27
#define PATH_TRACER_DEBUG_VIZ_HITKIND                   28
#define PATH_TRACER_DEBUG_VIZ_TRANSPARENCY_COUNT		29
#define PATH_TRACER_DEBUG_VIZ_SSS_COLOR                 30
#define PATH_TRACER_DEBUG_VIZ_SSS_RADIUS                31
#define PATH_TRACER_DEBUG_VIZ_SSS_WEIGHT                32
#define PATH_TRACER_DEBUG_VIZ_SSS_PHASE                 33
#define PATH_TRACER_DEBUG_VIZ_FUZZ_COLOR                34
#define PATH_TRACER_DEBUG_VIZ_FUZZ_ROUGHNESS            35
#define PATH_TRACER_DEBUG_VIZ_FUZZ_AMOUNT               36
#define PATH_TRACER_DEBUG_VIZ_BSDF_OPACITY              37
#define PATH_TRACER_DEBUG_VIZ_SUBSTRATE_WEIGHT_V		38
#define PATH_TRACER_DEBUG_VIZ_SUBSTRATE_TRANSMITTANCE_N 39
#define PATH_TRACER_DEBUG_VIZ_SUBSTRATE_COVERAGE_ABOVE_N 40
#define PATH_TRACER_DEBUG_VIZ_SUBSTRATE_F90 			41
#define PATH_TRACER_DEBUG_VIZ_EYE_IRIS_MASK             42
#define PATH_TRACER_DEBUG_VIZ_EYE_CAUSTIC_NORMAL        43
#define PATH_TRACER_DEBUG_VIZ_EYE_IRIS_NORMAL           44
#define PATH_TRACER_DEBUG_VIZ_BACKFACING_NORMAL         45
#define PATH_TRACER_DEBUG_VIZ_HAIR_PRIMITIVE_UV         46
#define PATH_TRACER_DEBUG_VIZ_TRIANGLE_ID               47
#define PATH_TRACER_DEBUG_VIZ_INSTANCE_ID               48
#define PATH_TRACER_DEBUG_VIZ_CLUSTER_ID                49
#define PATH_TRACER_DEBUG_VIZ_DECAL_COVERAGE            50
#define PATH_TRACER_DEBUG_VIZ_DECAL_COLOR               51
#define PATH_TRACER_DEBUG_VIZ_DECAL_NORMAL              52
#define PATH_TRACER_DEBUG_VIZ_DECAL_RMS                 53
#define PATH_TRACER_DEBUG_VIZ_DECAL_RADIANCE            54
#define PATH_TRACER_DEBUG_VIZ_DECAL_RESPONSE_MASK       55
#define PATH_TRACER_DEBUG_VIZ_DECAL_USE_DBUFFER         56
