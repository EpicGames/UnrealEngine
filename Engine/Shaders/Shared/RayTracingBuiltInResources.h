// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
	RayTracingBuiltInResources.h: used in ray tracing shaders and C++ code to define resources 
	available in all hit groups, such as root nostants, index and vertex buffers.
	!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#pragma once

#include "RayTracingDefinitions.h"
#include "HLSLReservedSpaces.h"

#if defined(__cplusplus)
	#define INCLUDED_FROM_CPP_CODE  1
	#define INCLUDED_FROM_HLSL_CODE 0
#else
	#define INCLUDED_FROM_CPP_CODE  0
	#define INCLUDED_FROM_HLSL_CODE 1
#endif

#if INCLUDED_FROM_HLSL_CODE
	#define UINT_TYPE uint
#elif INCLUDED_FROM_CPP_CODE
	#define UINT_TYPE unsigned int
#endif

struct FHitGroupSystemRootConstants
{
	// Config is a bitfield:
	// uint IndexStride  : 8; // Can be just 1 bit to indicate 16 or 32 bit indices
	// uint VertexStride : 8; // Can be just 2 bits to indicate float3, float2 or half2 format
	// uint Unused       : 16;
	UINT_TYPE Config;

	// Offset into HitGroupSystemIndexBuffer
	UINT_TYPE IndexBufferOffsetInBytes;

	// First primitive of the segment (as set in FRayTracingGeometrySegment)
	UINT_TYPE FirstPrimitive;

	// User-provided constant assigned to the hit group
	UINT_TYPE UserData;

	// Helper functions

	UINT_TYPE GetIndexStride()
	{
		return Config & 0xFF;
	}

	UINT_TYPE GetVertexStride()
	{
		return (Config >> 8) & 0xFF;
	}

	#if INCLUDED_FROM_CPP_CODE
		void SetVertexAndIndexStride(UINT_TYPE Vertex, UINT_TYPE Index)
		{
			Config = (Index & 0xFF) | ((Vertex & 0xFF) << 8);
		}
	#endif
};

#define RAY_TRACING_SYSTEM_INDEXBUFFER_REGISTER  0
#define RAY_TRACING_SYSTEM_VERTEXBUFFER_REGISTER 1
#define RAY_TRACING_SYSTEM_ROOTCONSTANT_REGISTER 0

#ifndef OVERRIDE_RAY_TRACING_HIT_GROUP_SYSTEM_RESOURCES
#define OVERRIDE_RAY_TRACING_HIT_GROUP_SYSTEM_RESOURCES 0
#endif

#if INCLUDED_FROM_HLSL_CODE && !OVERRIDE_RAY_TRACING_HIT_GROUP_SYSTEM_RESOURCES
	// Built-in local root parameters that are always bound to all hit shaders
	ByteAddressBuffer									HitGroupSystemIndexBuffer   : UE_HLSL_REGISTER(t, RAY_TRACING_SYSTEM_INDEXBUFFER_REGISTER,  UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
	ByteAddressBuffer									HitGroupSystemVertexBuffer  : UE_HLSL_REGISTER(t, RAY_TRACING_SYSTEM_VERTEXBUFFER_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
	ConstantBuffer<FHitGroupSystemRootConstants>		HitGroupSystemRootConstants : UE_HLSL_REGISTER(b, RAY_TRACING_SYSTEM_ROOTCONSTANT_REGISTER, UE_HLSL_SPACE_RAY_TRACING_SYSTEM);
#endif


#undef INCLUDED_FROM_CPP_CODE
#undef INCLUDED_FROM_HLSL_CODE
#undef UINT_TYPE

