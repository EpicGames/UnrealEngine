// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*================================================================================================
	RayTracingDebugTypes.h: used in ray tracing shaders and C++ code to define common types
	!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#ifdef __cplusplus
#define UINT32_TYPE uint32
#define UINT64_TYPE uint64
#else
#define UINT32_TYPE uint
#define UINT64_TYPE uint64_t
#endif

struct FRayTracingInstanceDebugData
{
	UINT32_TYPE Flags;
	UINT32_TYPE ProxyHash;
	UINT64_TYPE GeometryAddress;

#ifdef __cplusplus
	FRayTracingInstanceDebugData()
		: Flags(0)
		, ProxyHash(0)
		, GeometryAddress(0)
	{}
#endif
};

struct FRayTracingInstanceExtraData
{
	UINT32_TYPE SceneInstanceIndex;
	UINT32_TYPE GPUSceneInstanceId;

#ifdef __cplusplus
	FRayTracingInstanceExtraData()
		: SceneInstanceIndex(-1)
		, GPUSceneInstanceId(-1)
	{}
#endif
};

struct FRayTracingPickingFeedback
{
	// Hit data
	UINT32_TYPE InstanceIndex;
	UINT32_TYPE GeometryIndex;
	UINT32_TYPE TriangleIndex;

	// Geometry data
	UINT64_TYPE GeometryAddress; // TODO: Usage GeometryHandle instead

	// Instance data
	UINT32_TYPE InstanceId;
	UINT32_TYPE Mask;
	UINT32_TYPE InstanceContributionToHitGroupIndex;
	UINT32_TYPE Flags;

#ifdef __cplusplus
	FRayTracingPickingFeedback()
		: InstanceIndex(0xFFFFFFFF)
		, GeometryAddress(0xFFFFFFFFFFFFFFFF)
	{}
#endif
};

struct FRayTracingHitStatsEntry
{
	UINT32_TYPE PrimitiveID;
	UINT32_TYPE Count;

#ifdef __cplusplus
	FRayTracingHitStatsEntry()
	{}
#endif
};

#undef UINT32_TYPE
#undef UINT64_TYPE
