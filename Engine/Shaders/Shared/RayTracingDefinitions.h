// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
	RayTracingDefinitions.h: used in ray tracing shaders and C++ code to define common constants
	!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#pragma once

#include "HLSLReservedSpaces.h"

// Change this to force recompilation of all ray tracing shaders (use https://www.random.org/cgi-bin/randbyte?nbytes=4&format=h)
// This avoids changing the global ShaderVersion.ush and forcing recompilation of all shaders in the engine (only RT shaders will be affected)
#define RAY_TRACING_SHADER_VERSION 0x500880FC 

#define RAY_TRACING_MASK_OPAQUE						0x01    // Opaque and alpha tested meshes and particles (e.g. used by reflection, shadow, AO and GI tracing passes)
#define RAY_TRACING_MASK_TRANSLUCENT				0x02    // Opaque and alpha tested meshes and particles (e.g. used by translucency tracing pass)
#define RAY_TRACING_MASK_OPAQUE_SHADOW				0x04    // Opaque and alpha tested geometry visible for shadow rays
#define RAY_TRACING_MASK_TRANSLUCENT_SHADOW			0x08    // Translucent geometry visible for shadow rays
#define RAY_TRACING_MASK_THIN_SHADOW				0x10    // Whether the thin geometry (e.g. hair) is visible for shadow rays
#define RAY_TRACING_MASK_OPAQUE_REFLECTION			0x20	// Opaque geometry visible in reflections
#define RAY_TRACING_MASK_HAIR_STRANDS               0x40    // For primary ray tracing against hair
#define RAY_TRACING_MASK_OPAQUE_FP_WORLD_SPACE      0x80    // For filtering out world space representations of a first person mesh when tracing rays originating from a first person mesh
#define RAY_TRACING_MASK_ALL						0xFF

#define RAY_TRACING_MASK_SHADOW						(RAY_TRACING_MASK_OPAQUE_SHADOW | RAY_TRACING_MASK_TRANSLUCENT_SHADOW)

#define RAY_TRACING_SHADER_SLOT_MATERIAL	0
#define RAY_TRACING_SHADER_SLOT_SHADOW		1
#define RAY_TRACING_NUM_SHADER_SLOTS		2

#define RAY_TRACING_MISS_SHADER_SLOT_DEFAULT	0 // Main miss shader that simply sets HitT to a "miss" value (see FMinimalPayload::SetMiss())
#define RAY_TRACING_MISS_SHADER_SLOT_LIGHTING	1 // Miss shader that may be used to evaluate light source radiance in ray traced reflections and translucency
#define RAY_TRACING_NUM_MISS_SHADER_SLOTS		2

#define RAY_TRACING_LIGHT_COUNT_MAXIMUM		256
#define RAY_TRACING_DECAL_COUNT_MAXIMUM		256

#define RAY_TRACING_MAX_ALLOWED_RECURSION_DEPTH 1   // Only allow ray tracing from RayGen shader
#define RAY_TRACING_MAX_ALLOWED_ATTRIBUTE_SIZE  8   // Sizeof 2 floats (barycentrics)

// See enum EReflectedAndRefractedRayTracedShadows
#define RAY_TRACING_SHADOWS_TYPE_OFF       0
#define RAY_TRACING_SHADOWS_TYPE_HARD      1
#define RAY_TRACING_SHADOWS_TYPE_SOFT      2

#define RAY_TRACING_SHADOWS_TRANSLUCENCY_TYPE_OPAQUE 					0
#define RAY_TRACING_SHADOWS_TRANSLUCENCY_TYPE_MASKED 					1
#define RAY_TRACING_SHADOWS_TRANSLUCENCY_TYPE_FRACTIONAL_VISIBILITY 	2