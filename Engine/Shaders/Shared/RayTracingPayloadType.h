// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HLSLStaticAssert.h"

#define RT_PAYLOAD_TYPE_MINIMAL					(1 << 0)	// FMinimalPayload
#define RT_PAYLOAD_TYPE_DEFAULT					(1 << 1)	// FDefaultPayload
#define RT_PAYLOAD_TYPE_RAYTRACING_MATERIAL		(1 << 2)	// FPackedMaterialClosestHitPayload
#define RT_PAYLOAD_TYPE_RAYTRACING_DEBUG		(1 << 3)	// FRayTracingDebugPayload
#define RT_PAYLOAD_TYPE_DEFERRED				(1 << 4)	// FDeferredMaterialPayload
#define RT_PAYLOAD_TYPE_PATHTRACING_MATERIAL	(1 << 5)	// FPackedPathTracingPayload
#define RT_PAYLOAD_TYPE_LUMEN_MINIMAL			(1 << 6)	// FLumenMinimalPayload
#define RT_PAYLOAD_TYPE_VFX   					(1 << 7)	// FVFXTracePayload
#define RT_PAYLOAD_TYPE_DECALS					(1 << 8)	// FDecalShaderPayload 
#define RT_PAYLOAD_TYPE_SPARSE_VOXEL			(1 << 9)	// FSparseVoxelPayload
#define RT_PAYLOAD_TYPE_GPULIGHTMASS            (1 << 10)   // FPackedPathTracingPayload

#ifdef __cplusplus
/**
* Represent which payload type a given raytracing shader can use. This is a bitfield because raygen shaders may
  trace various types of rays, or use different kind of callable shaders.
  HitGroup, Miss and Callable shaders must (by definition) only use a single payload type.

  This payload type must be provided via the GetRayTracingPayloadType static member function when implementing a ray tracing shader.
*/
enum class ERayTracingPayloadType : uint32 // C++
#else
enum ERayTracingPayloadType : uint // HLSL
#endif
{
	None = 0, // placeholder for non-raytracing shaders
	Minimal = RT_PAYLOAD_TYPE_MINIMAL,
	Default = RT_PAYLOAD_TYPE_DEFAULT,
	RayTracingMaterial = RT_PAYLOAD_TYPE_RAYTRACING_MATERIAL,
	RayTracingDebug = RT_PAYLOAD_TYPE_RAYTRACING_DEBUG,
	Deferred = RT_PAYLOAD_TYPE_DEFERRED,
	PathTracingMaterial = RT_PAYLOAD_TYPE_PATHTRACING_MATERIAL,
	LumenMinimal = RT_PAYLOAD_TYPE_LUMEN_MINIMAL,
	VFX = RT_PAYLOAD_TYPE_VFX,
	Decals = RT_PAYLOAD_TYPE_DECALS,
	SparseVoxel = RT_PAYLOAD_TYPE_SPARSE_VOXEL,
	GPULightmass = RT_PAYLOAD_TYPE_GPULIGHTMASS,
};


#ifdef RT_PAYLOAD_TYPE

// If the shader was provided a RT_PAYLOAD_TYPE, we test its bitmask here
#define IS_PAYLOAD_ENABLED(T)    ((RT_PAYLOAD_TYPE & (T)) != 0)

#else

// If no payload type was defined, we can't possibly have any enabled
#define IS_PAYLOAD_ENABLED(T)    (0)

#endif


#ifdef RT_PAYLOAD_MAX_SIZE

#define CHECK_RT_PAYLOAD_SIZE(StructName)		HLSL_STATIC_ASSERT(sizeof(StructName) <= RT_PAYLOAD_MAX_SIZE, "Payload " #StructName " is larger than expected");

#else

#define CHECK_RT_PAYLOAD_SIZE(StructName)		

#endif