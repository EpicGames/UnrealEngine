// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/*================================================================================================
	RayTracingTypes.h: used in ray tracing shaders and C++ code to define common types
	!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#include "HLSLStaticAssert.h"

#ifdef __cplusplus
#include "HLSLTypeAliases.h"

namespace UE::HLSL
{
#endif

struct FRTLightingData
{
	uint   Type;
	float  IESAtlasIndex;
	float  RectLightAtlasMaxLevel;
	uint   LightMissShaderIndex;
	float3 TranslatedLightPosition;
	float  InvRadius;
	float3 Direction;
	float  FalloffExponent;
	float3 LightColor;
	uint   DiffuseSpecularScale;
	float3 Tangent;
	float  SourceRadius;
	float2 SpotAngles;
	float  SourceLength;
	float  SoftSourceRadius;
	uint   DistanceFadeMAD;
	float  IndirectLightScale;
	float  RectLightBarnCosAngle;
	float  RectLightBarnLength;
	float2 RectLightAtlasUVOffset;
	float2 RectLightAtlasUVScale;
};
HLSL_STATIC_ASSERT(sizeof(FRTLightingData) == 128, "Ray tracing light structure should be kept as small as possible");

// #dxr_todo: Unify this with FRTLightingData ?
struct FPathTracingLight
{
	float3  TranslatedWorldPosition;
	float3  Normal;
	float3  Tangent;
	float3  Color;
	float2  Dimensions; // Radius,Length or RectWidth,RectHeight or Sin(Angle/2),0 depending on light type
	float2  Shaping;    // Barndoor controls for RectLights, Cone angles for spot lights
	uint    DiffuseSpecularScale;
	float   IndirectLightingScale;
	float   Attenuation;
	float   FalloffExponent; // for non-inverse square decay lights only
	float   VolumetricScatteringIntensity;  // scale for volume contributions
	float   IESAtlasIndex;
	uint    Flags; // see defines PATHTRACER_FLAG_*
	uint    MissShaderIndex;  // used to implement light functions
	float2  RectLightAtlasUVScale;  // Rect. light atlas UV transformation
	float2  RectLightAtlasUVOffset; // Rect. light atlas UV transformation
	float3  CameraColorScale; // Tinting factor for camera rays only
	uint    Padding;
};
HLSL_STATIC_ASSERT(sizeof(FPathTracingLight) == 128, "Path tracing light structure should be kept as small as possible");

struct FPathTracingPackedPathState
{
	uint      RandSeqSampleIndex;
	uint      RandSeqSampleSeed;
	float3    Radiance;
	float     Alpha;
	uint3     PackedAlbedoNormal;
	float3    RayOrigin;
	float3    RayDirection;
	float3    PathThroughput;
	uint2     PackedRoughnessSigma;
	uint2     PackedScatterPhase;
};
HLSL_STATIC_ASSERT(sizeof(FPathTracingPackedPathState) == 88, "Packed Path State size should be minimized");

struct FPathTracingDebugPackedPathState
{
	uint      RandSeqSampleIndex;
	uint      RandSeqSampleSeed;
	float3    Radiance;
	float     Alpha;
	uint3     PackedDiffSpecColor;
	uint2     PackedNormalRoughness;
	float3    RayOrigin;
	float3    RayDirection;
	float3    PathThroughput;
	uint2     PackedRoughnessSigma;
	uint2     PackedScatterPhase;
};
HLSL_STATIC_ASSERT(sizeof(FPathTracingDebugPackedPathState) == 96, "Packed Path State size should be minimized");


struct FRayTracingDecal
{
	float3 		TranslatedBoundMin;
	uint   		Pad0; // keep structure aligned
	float3 		TranslatedBoundMax;
	uint   		CallableSlotIndex;
	float4x4	TranslatedWorldToDecal;
};
HLSL_STATIC_ASSERT(sizeof(FRayTracingDecal) == 96, "Ray tracing decal structure should be aligned to 32 bytes for optimal access on the GPU");

#ifdef __cplusplus
} // namespace UE::HLSL

using FRTLightingData = UE::HLSL::FRTLightingData;
using FPathTracingLight = UE::HLSL::FPathTracingLight;
using FPathTracingPackedPathState = UE::HLSL::FPathTracingPackedPathState;
using FPathTracingDebugPackedPathState = UE::HLSL::FPathTracingDebugPackedPathState;
using FRayTracingDecal = UE::HLSL::FRayTracingDecal;

#endif
