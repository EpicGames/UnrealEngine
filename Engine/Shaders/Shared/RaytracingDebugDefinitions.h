// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
	RayTracingDebugDefinitions.ush: used in ray tracing debug shaders and C++ code to define common constants
	!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#pragma once

#define RAY_TRACING_DEBUG_VIZ_RADIANCE							0 
#define RAY_TRACING_DEBUG_VIZ_WORLD_NORMAL						1 
#define RAY_TRACING_DEBUG_VIZ_BASE_COLOR						2 
#define RAY_TRACING_DEBUG_VIZ_DIFFUSE_COLOR						3 
#define RAY_TRACING_DEBUG_VIZ_SPECULAR_COLOR					4 
#define RAY_TRACING_DEBUG_VIZ_OPACITY							5 
#define RAY_TRACING_DEBUG_VIZ_METALLIC							6 
#define RAY_TRACING_DEBUG_VIZ_SPECULAR							7 
#define RAY_TRACING_DEBUG_VIZ_ROUGHNESS							8 
#define RAY_TRACING_DEBUG_VIZ_IOR								9 
#define RAY_TRACING_DEBUG_VIZ_SHADING_MODEL						10 
#define RAY_TRACING_DEBUG_VIZ_BLENDING_MODE						11  
#define RAY_TRACING_DEBUG_VIZ_LIGHTING_CHANNEL_MASK				12  
#define RAY_TRACING_DEBUG_VIZ_CUSTOM_DATA						13  
#define RAY_TRACING_DEBUG_VIZ_GBUFFER_AO						14  
#define RAY_TRACING_DEBUG_VIZ_INDIRECT_IRRADIANCE				15  
#define RAY_TRACING_DEBUG_VIZ_WORLD_POSITION					16  
#define RAY_TRACING_DEBUG_VIZ_HITKIND							17  
#define RAY_TRACING_DEBUG_VIZ_BARYCENTRICS						18  
#define RAY_TRACING_DEBUG_VIZ_PRIMARY_RAYS						19  
#define RAY_TRACING_DEBUG_VIZ_WORLD_TANGENT						20 
#define RAY_TRACING_DEBUG_VIZ_ANISOTROPY						21 
#define RAY_TRACING_DEBUG_VIZ_INSTANCES							22
#define RAY_TRACING_DEBUG_VIZ_TIMING_TRAVERSAL					23 
#define RAY_TRACING_DEBUG_VIZ_TIMING_ANY_HIT					24 
#define RAY_TRACING_DEBUG_VIZ_TIMING_MATERIAL					25 
#define RAY_TRACING_DEBUG_VIZ_TRIANGLES							26
#define RAY_TRACING_DEBUG_VIZ_CLUSTERS							27
#define RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_NODE			28
#define RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_CLUSTER			29
#define RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_TRIANGLE		30
#define RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_ALL				31
#define RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_STATISTICS		32
#define RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_NODE			33
#define RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_CLUSTER		34
#define RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_TRIANGLE		35
#define RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_ALL			36
#define RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_STATISTICS	37
#define RAY_TRACING_DEBUG_VIZ_DYNAMIC_INSTANCES					38
#define RAY_TRACING_DEBUG_VIZ_PROXY_TYPE						39
#define RAY_TRACING_DEBUG_VIZ_PICKER							40
#define RAY_TRACING_DEBUG_VIZ_INSTANCE_OVERLAP					41
#define RAY_TRACING_DEBUG_VIZ_TRIANGLE_HITCOUNT					42
#define RAY_TRACING_DEBUG_VIZ_HITCOUNT_PER_INSTANCE				43
#define RAY_TRACING_DEBUG_VIZ_LIGHT_GRID_COUNT					44
#define RAY_TRACING_DEBUG_VIZ_SUBSTRATE_DATA					45

#define RAY_TRACING_DEBUG_PICKER_DOMAIN_TRIANGLE				0
#define RAY_TRACING_DEBUG_PICKER_DOMAIN_INSTANCE				1
#define RAY_TRACING_DEBUG_PICKER_DOMAIN_SEGMENT					2
#define RAY_TRACING_DEBUG_PICKER_DOMAIN_FLAGS                   3
#define RAY_TRACING_DEBUG_PICKER_DOMAIN_MASK                    4

#if defined(__cplusplus)
	#define UINT_TYPE unsigned int
	#define INT_TYPE int
	#define INLINE_ATTR inline
#else
	#define UINT_TYPE uint
	#define INT_TYPE int
	#define INLINE_ATTR 
#endif

INLINE_ATTR bool RequiresRayTracingDebugCHS(UINT_TYPE DebugVisualizationMode)
{
	return DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_INSTANCES
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRIANGLES
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_CLUSTERS
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_DYNAMIC_INSTANCES
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_PROXY_TYPE
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_PICKER
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_INSTANCE_OVERLAP
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRIANGLE_HITCOUNT
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_HITCOUNT_PER_INSTANCE;
}

INLINE_ATTR bool IsRayTracingDebugTraversalMode(UINT_TYPE DebugVisualizationMode)
{
	return DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_NODE
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_CLUSTER
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_TRIANGLE
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_ALL
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_PRIMARY_STATISTICS
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_NODE
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_CLUSTER
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_TRIANGLE
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_ALL
		|| DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_TRAVERSAL_SECONDARY_STATISTICS;
}

INLINE_ATTR bool RayTracingDebugModeSupportsInline(UINT_TYPE DebugVisualizationMode)
{
	const bool bIsBarycentricsMode = DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_BARYCENTRICS;
	const bool bIsWorldNormalMode = DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_WORLD_NORMAL;
	const bool bUsesDebugCHSAndNotHitCount = RequiresRayTracingDebugCHS(DebugVisualizationMode)
		&& (DebugVisualizationMode != RAY_TRACING_DEBUG_VIZ_TRIANGLE_HITCOUNT)
		&& (DebugVisualizationMode != RAY_TRACING_DEBUG_VIZ_HITCOUNT_PER_INSTANCE);
	const bool bIsTraversalMode = IsRayTracingDebugTraversalMode(DebugVisualizationMode);
	const bool bIsPickingMode = DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_PICKER;
	const bool bIsHitKind = DebugVisualizationMode == RAY_TRACING_DEBUG_VIZ_HITKIND;

	return bIsBarycentricsMode || bIsWorldNormalMode || bIsPickingMode || bUsesDebugCHSAndNotHitCount || bIsTraversalMode || bIsHitKind;
}
