// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Number of curves per cluster
#define RENDERCURVE_CURVE_PER_CLUSTER	64u

// Primitive data size
#define RENDERCURVE_PRIMITIVE_DATA_STRIDE_IN_BYTES 16