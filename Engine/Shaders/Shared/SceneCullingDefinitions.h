// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// 1 + 7 + 24 bits_
#define  INSTANCE_HIERARCHY_ITEM_CHUNK_COUNT_SHIFT  (24u)
#define  INSTANCE_HIERARCHY_ITEM_CHUNK_ID_MASK  ((1u << INSTANCE_HIERARCHY_ITEM_CHUNK_COUNT_SHIFT) - 1u)
#define  INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_FLAG  (1u << 31u)
#define  INSTANCE_HIERARCHY_ITEM_CHUNK_COMPRESSED_PAYLOAD_MASK  ((1u << 31u) - 1u)
#define  INSTANCE_HIERARCHY_MAX_CHUNK_SIZE  (64u)

#define  INSTANCE_HIERARCHY_CELL_HEADER_OFFSET_BITS (22u) // 22 for offset
#define  INSTANCE_HIERARCHY_CELL_HEADER_COUNT_BITS (21u)  // 2x21 for static/dynamic

#ifdef __cplusplus
#include "HLSLTypeAliases.h"

namespace UE::HLSL
{
#endif
#ifndef __cplusplus //HLSL
#include "/Engine/Private/LargeWorldCoordinates.ush"
#endif

// Representation in hierarchy buffer
struct FPackedCellHeader
{
	uint Packed0;
	uint Packed1;
};

// Unpacked version of the struct
struct FCellHeader
{
	uint NumItemChunks;
	uint ItemChunksOffset;
	uint NumStaticChunks;
	uint NumDynamicChunks;
	bool bIsValid;
};

struct FCellBlockData
{
	FDFVector3 WorldPos;
	float LevelCellSize; // Note, not the block size, but the cell size.
	uint Pad;
};

/**
 * Represent one item of work for the hierarchical culling stage, linking a cell to either a group of views (main-pass) or a singlular view (post-pass).
 */
struct FCellChunkDraw
{
	uint ItemChunksOffset;
	uint ViewGroupId;
};

/**
 * Represents a group of views, e.g., for a clipmap or point light, or anything else really that share the same broad-phase culling result.
 * Wrt mip-views, there is no explicit handling in the hierarchical culling stage, as they are expected to come in a compact range (post view compaction)
 * A view group should typically share view flags, might want/need to make assumptions around that.
 */
struct FViewDrawGroup
{
	uint FirstView;
	uint NumViews;
};

// Info for one instance culling workgroup (64 threads).
// TODO: Pack/unpack into fewer bits?
// TODO: Move to some nanite specific header probably.
struct FInstanceCullingGroupWork
{
	uint ViewGroupId;
	uint PackedItemChunkDesc;
	uint ActiveViewMask; // Up to 32 active views in the group (NOTE: this may overflow for example if all mip levels were mapped at the same time on a point light, 48 mips)
};


#define SIZEOF_PACKED_CHUNK_ATTRIBUTES 20u
/**
 */
struct FPackedChunkAttributes
{
	uint2 Aabb;
	float2 InstanceDrawDistanceMinMaxSquared;
	float MinAnimationMinScreenSize;
};

/**
 */
struct FOccludedChunkDraw
{
	uint ViewGroupId;
	uint OccludedViewMask;
	uint ChunkId;
};

#ifdef __cplusplus
} // namespace

using FCellHeader = UE::HLSL::FCellHeader;
using FPackedCellHeader = UE::HLSL::FPackedCellHeader;
using FCellBlockData = UE::HLSL::FCellBlockData;
using FCellChunkDraw = UE::HLSL::FCellChunkDraw;
using FViewDrawGroup = UE::HLSL::FViewDrawGroup;
using FInstanceCullingGroupWork = UE::HLSL::FInstanceCullingGroupWork;
using FPackedChunkAttributes = UE::HLSL::FPackedChunkAttributes;
using FOccludedChunkDraw = UE::HLSL::FOccludedChunkDraw;

#endif
