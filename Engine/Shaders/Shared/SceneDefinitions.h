// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
!!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#pragma once

#define ENABLE_SCENE_DATA_DX11_UB_ERROR_WORKAROUND 1

// GPUCULL_TODO: Eventually we need to remove this workaround
#define VF_TREAT_INSTANCE_ID_OFFSET_AS_PRIMITIVE_ID_FLAG (1U << 31U)

#define PRIMITIVE_ID_NUM_BITS				(20u) // Max of 1,048,576 primitives
#define INSTANCE_ID_NUM_BITS				(24u) // Max of 16,777,216 instances in the buffer
#define INSTANCE_SCENE_DATA_FLAGS_NUM_BITS	(12u) // Max of 12 flags
#define INSTANCE_RELATIVE_ID_NUM_BITS		(24u) // Max of 16,777,216 instances per primitive
#define INSTANCE_CUSTOM_DATA_COUNT_NUM_BITS	 (8u) // Max of 255 custom data floats per instance
#define LIGHT_ID_NUM_BITS					(23u) // Max of 8,388,608 lights in the buffer

#define MAX_PRIMITIVE_ID					(1U << PRIMITIVE_ID_NUM_BITS)
#define MAX_INSTANCE_ID						(1U << INSTANCE_ID_NUM_BITS)
#define MAX_LIGHT_ID						(1U << LIGHT_ID_NUM_BITS)

#define PRIMITIVE_ID_MASK			((1u << PRIMITIVE_ID_NUM_BITS) - 1u)
#define INSTANCE_ID_MASK			((1u << INSTANCE_ID_NUM_BITS) - 1u)
#define INSTANCE_RELATIVE_ID_MASK	((1u << INSTANCE_RELATIVE_ID_NUM_BITS) - 1u)
#define LIGHT_ID_MASK				((1u << LIGHT_ID_NUM_BITS) - 1u)

#define INVALID_PRIMITIVE_ID PRIMITIVE_ID_MASK

// Primitive scene data flags
#define PRIMITIVE_SCENE_DATA_FLAG_CAST_SHADOWS								0x1
#define PRIMITIVE_SCENE_DATA_FLAG_USE_SINGLE_SAMPLE_SHADOW_SL				0x2
#define PRIMITIVE_SCENE_DATA_FLAG_USE_VOLUMETRIC_LM_SHADOW_SL				0x4
#define PRIMITIVE_SCENE_DATA_FLAG_DECAL_RECEIVER							0x8
#define PRIMITIVE_SCENE_DATA_FLAG_UNUSED_1									0x10
#define PRIMITIVE_SCENE_DATA_FLAG_OUTPUT_VELOCITY							0x20
#define PRIMITIVE_SCENE_DATA_FLAG_DETERMINANT_SIGN							0x40
#define PRIMITIVE_SCENE_DATA_FLAG_HAS_RAY_TRACING_REPRESENTATION			0x80
#define PRIMITIVE_SCENE_DATA_FLAG_HAS_CAPSULE_REPRESENTATION				0x100
#define PRIMITIVE_SCENE_DATA_FLAG_HAS_CAST_CONTACT_SHADOW					0x200
#define PRIMITIVE_SCENE_DATA_FLAG_HAS_PRIMITIVE_CUSTOM_DATA					0x400
#define PRIMITIVE_SCENE_DATA_FLAG_LIGHTING_CHANNEL_0						0x800
#define PRIMITIVE_SCENE_DATA_FLAG_LIGHTING_CHANNEL_1						0x1000
#define PRIMITIVE_SCENE_DATA_FLAG_LIGHTING_CHANNEL_2						0x2000
#define PRIMITIVE_SCENE_DATA_FLAG_HAS_INSTANCE_LOCAL_BOUNDS					0x4000
#define PRIMITIVE_SCENE_DATA_FLAG_EVALUATE_WORLD_POSITION_OFFSET			0x8000
#define PRIMITIVE_SCENE_DATA_FLAG_UNUSED_2									0x10000
#define PRIMITIVE_SCENE_DATA_FLAG_INSTANCE_DRAW_DISTANCE_CULL				0x20000
#define PRIMITIVE_SCENE_DATA_FLAG_WPO_DISABLE_DISTANCE						0x40000
#define PRIMITIVE_SCENE_DATA_FLAG_HAS_ALWAYS_EVALUATE_WPO_MATERIALS			0x80000
#define PRIMITIVE_SCENE_DATA_FLAG_UNUSED_3									0x100000
#define PRIMITIVE_SCENE_DATA_FLAG_DISABLE_MATERIAL_INVALIDATIONS			0x200000
#define PRIMITIVE_SCENE_DATA_FLAG_HOLDOUT									0x400000
#define PRIMITIVE_SCENE_DATA_FLAG_SPLINE_MESH								0x800000
#define PRIMITIVE_SCENE_DATA_FLAG_SKINNED_MESH								0x1000000
#define PRIMITIVE_SCENE_DATA_FLAG_INSTANCE_CULLING_OCCLUSION_QUERIES		0x2000000 // Whether per-instance occlusion queries are allowed when performing GPUScene instance culling
#define PRIMITIVE_SCENE_DATA_FLAG_HAS_PIXEL_ANIMATION						0x4000000
#define PRIMITIVE_SCENE_DATA_FLAG_RAYTRACING_FAR_FIELD						0x8000000
#define PRIMITIVE_SCENE_DATA_FLAG_RAYTRACING_HAS_GROUPID					0x10000000
#define PRIMITIVE_SCENE_DATA_FLAG_PER_CLUSTER_DISPLACEMENT_FALLBACK_RASTER	0x20000000
#define PRIMITIVE_SCENE_DATA_FLAG_IS_FIRST_PERSON							0x40000000


// Primitive visibility flags
#define PRIMITIVE_VISIBILITY_FLAG_FORCE_HIDDEN							0x1
#define PRIMITIVE_VISIBILITY_FLAG_CAST_HIDDEN_SHADOW					0x2
#define PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_GAME						0x4
#define PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_EDITOR						0x8
#define PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_MAIN_PASS					0x10
#define PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_REFLECTION_CAPTURES		0x20
#define PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_REAL_TIME_SKY_CAPTURES		0x40
#define PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_RAY_TRACING				0x80
#define PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_SCENE_CAPTURE_ONLY			0x100
#define PRIMITIVE_VISIBILITY_FLAG_HIDDEN_IN_SCENE_CAPTURE				0x200
#define PRIMITIVE_VISIBILITY_FLAG_VISIBLE_IN_LUMEN_SCENE				0x400

// NOTE: Reverse culling in Nanite was implemented, but found to create issues with HLOD/PLA that rely on the flag for certain
// behavior with non-Nanite. Will need a resolution for these issues before we're able to add support back. See UE-174319
#define SUPPORT_REVERSE_CULLING_IN_NANITE 0

#if SUPPORT_REVERSE_CULLING_IN_NANITE
#define PRIMITIVE_SCENE_DATA_FLAG_REVERSE_CULLING						NOTE: Out of flags
#endif
// Stride of a single primitive's data in float4's
#define PRIMITIVE_SCENE_DATA_STRIDE 44

// Currently limited by # of bits for size in primitive scene data
#define PRIMITIVE_SCENE_DATA_MAX_PAYLOAD_EXTENSION_SIZE 255

// Instance scene data flags
#define INSTANCE_SCENE_DATA_FLAG_DETERMINANT_SIGN			0x1
#define INSTANCE_SCENE_DATA_FLAG_HAS_RANDOM					0x2
#define INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA			0x4
#define INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA			0x8
#define INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA			0x10
#define INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS	0x20
#define INSTANCE_SCENE_DATA_FLAG_HAS_HIERARCHY_OFFSET		0x40
#define INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS			0x80
#define INSTANCE_SCENE_DATA_FLAG_HAS_PAYLOAD_EXTENSION		0x100
#define INSTANCE_SCENE_DATA_FLAG_HAS_EDITOR_DATA			0x200
#define INSTANCE_SCENE_DATA_FLAG_HIDDEN						0x400

#define INSTANCE_SCENE_DATA_FLAG_PAYLOAD_MASK (			\
	INSTANCE_SCENE_DATA_FLAG_HAS_RANDOM					\
	| INSTANCE_SCENE_DATA_FLAG_HAS_CUSTOM_DATA			\
	| INSTANCE_SCENE_DATA_FLAG_HAS_DYNAMIC_DATA			\
	| INSTANCE_SCENE_DATA_FLAG_HAS_SKINNING_DATA		\
	| INSTANCE_SCENE_DATA_FLAG_HAS_LIGHTSHADOW_UV_BIAS	\
	| INSTANCE_SCENE_DATA_FLAG_HAS_HIERARCHY_OFFSET		\
	| INSTANCE_SCENE_DATA_FLAG_HAS_LOCAL_BOUNDS			\
	| INSTANCE_SCENE_DATA_FLAG_HAS_EDITOR_DATA			\
	| INSTANCE_SCENE_DATA_FLAG_HAS_PAYLOAD_EXTENSION	\
)

// Specific to "UniformView" rendering path
#define BATCHED_PRIMITIVE_DATA_STRIDE 512u
#define BATCHED_INSTANCE_DATA_STRIDE 256u
#define BATCHED_PRIMITIVE_DATA_STRIDE_FLOAT4 (BATCHED_PRIMITIVE_DATA_STRIDE/16u)
#define BATCHED_INSTANCE_DATA_STRIDE_FLOAT4 (BATCHED_INSTANCE_DATA_STRIDE/16u)