// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
	SingleLayerWaterDefinitions.h: 
	Used in shaders and C++ code to define common constants
	!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/
#pragma once

#define SLW_TILE_SIZE_XY (8U)