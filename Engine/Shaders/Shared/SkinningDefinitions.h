// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define USE_COMPRESSED_BONE_TRANSFORM 0

#define GPUSKIN_TRANSFORM_STORAGE_BONE_MAP 0
#define GPUSKIN_TRANSFORM_STORAGE_DIRECT   1

#ifdef __cplusplus
#define SKINNING_STATIC_ASSERT(bCondition, Message) static_assert(bCondition, Message)
#else
#define SKINNING_STATIC_ASSERT(bCondition, Message)
#endif

#if defined(__cplusplus) || COMPILER_SUPPORTS_HLSL2021

#ifdef __cplusplus

#include "HLSLTypeAliases.h"

namespace UE::HLSL
{
#endif

struct FAnimBankBlockHeader
{
	float Time;
	uint BoneMapOffset;
	uint SequenceTransformIndex;
	uint MappingTransformIndex;
	uint OutputTransformIndex;
	uint FrameCount;
	uint BoneCount                      : 16;
	uint OutputTransformCopyIndexOffset : 16;
	uint BlockLocalIndex                : 16;
	uint BlockTransformCount            : 16;
};

struct FBoneRetargetingData
{
	uint AnimBoneIndex : 16;
	uint RetargetMode  : 15;
	uint bRequiresRefPoseRetarget : 1;
};
SKINNING_STATIC_ASSERT(sizeof(FBoneRetargetingData) == 4, "FBoneRetargetingData must be 4 bytes");

#define MAX_ANIM_SEQUENCE_LAYERS 4

#define SKINNING_LAYER_MODE_OVERRIDE 0
#define SKINNING_LAYER_MODE_ADDITIVE 1

#define SKINNING_MAX_KEY_INDEX 65535

#define SKINNING_LAYER_MODE_BITS_PER_LAYER 2
#define SKINNING_LAYER_MODE_MASK_PER_LAYER ((1 << SKINNING_LAYER_MODE_BITS_PER_LAYER) - 1)

#define SKINNING_INVALID_BONE_MASK_OFFSET 0x3FFFFFF

#define SKINNING_MAX_BONES_PER_GROUP 1024
#define SKINNING_MAX_BONES_PER_GROUP_MESHSPACE 256

// Per-layer header. One entry per layer per track.
struct FEvaluateLayerHeader
{
	uint  BoneMapBufferOffset;
	uint  DstTransformBufferOffset1;
	uint  DstTransformBufferOffset2;
	uint  SampleDataOffset;
	float BlendWeight;
	float LayerWeight;
	uint  BoneMaskAndSampleCounts;               // [0:25] BoneMask, [26:28] NumSourceSamples, [29:31] NumTargetSamples
	uint  InvGlobalRefPoseTransformBufferOffset;
};
SKINNING_STATIC_ASSERT(sizeof(FEvaluateLayerHeader) == 32, "FEvaluateLayerHeader must be 32 bytes");

// Per-sample pose evaluation data. Tightly packed per track.
struct FSamplePoseData
{
	float Alpha;
	uint  KeyIndex0;
	uint  KeyIndex1;
	uint  SequenceTransformBufferOffset;
	uint  RetargetingTransformBufferOffset;
	uint  RetargetingDataBufferOffset;
	uint  NumAnimBones;
	float Weight;
};
SKINNING_STATIC_ASSERT(sizeof(FSamplePoseData) == 32, "FSamplePoseData must be 32 bytes");

struct FBoneHierarchyData
{
	uint ParentIndex    : 16;
	uint TransformIndex : 16;
};
SKINNING_STATIC_ASSERT(sizeof(FBoneHierarchyData) == 4, "FBoneHierarchyData must be 4 bytes");

#ifdef __cplusplus
inline void SetBoneMaskAndSampleCounts(FEvaluateLayerHeader& H, uint32 BoneMask, uint32 NumSource, uint32 NumTarget)
{
	H.BoneMaskAndSampleCounts = (BoneMask & 0x3FFFFFF) | (NumSource << 26) | (NumTarget << 29);
}
#else
uint  GetBoneMaskOffset(FEvaluateLayerHeader H)      { return H.BoneMaskAndSampleCounts & 0x3FFFFFF; }
uint  GetNumSourceSamples(FEvaluateLayerHeader H)    { return (H.BoneMaskAndSampleCounts >> 26) & 0x7; }
uint  GetNumTargetSamples(FEvaluateLayerHeader H)    { return (H.BoneMaskAndSampleCounts >> 29) & 0x7; }
bool  HasBoneMask(FEvaluateLayerHeader H)             { return GetBoneMaskOffset(H) != SKINNING_INVALID_BONE_MASK_OFFSET; }
#endif

struct FResolveAttachmentBlockHeader
{
	uint ParentTransformBufferOffset;
	uint ParentMaxTransformCountAndBoneIndex;       // bits [15:0] = MaxTransformCount, [31:16] = BoneIndex
	uint ParentCurrentTransformSlotAndInstanceId;   // bits [7:0] = CurrentTransformSlot, [31:8] = ParentInstanceId
	uint ChildInstanceId;
	uint ChildPrimitiveId;
	float LocalOffsetTranslation[3];
	float LocalOffsetRotation[4];
	float BoneRefPoseRow0[4];
	float BoneRefPoseRow1[4];
	float BoneRefPoseRow2[4];
};
SKINNING_STATIC_ASSERT(sizeof(FResolveAttachmentBlockHeader) == 96, "FResolveAttachmentBlockHeader must be 96 bytes");

#ifdef __cplusplus
inline void SetMaxTransformCount(FResolveAttachmentBlockHeader& H, uint32 V)    { H.ParentMaxTransformCountAndBoneIndex = (H.ParentMaxTransformCountAndBoneIndex & 0xFFFF0000) | (V & 0xFFFF); }
inline void SetBoneIndex(FResolveAttachmentBlockHeader& H, uint32 V)            { H.ParentMaxTransformCountAndBoneIndex = (H.ParentMaxTransformCountAndBoneIndex & 0x0000FFFF) | (V << 16); }
inline void SetCurrentTransformSlot(FResolveAttachmentBlockHeader& H, uint32 V) { H.ParentCurrentTransformSlotAndInstanceId = (H.ParentCurrentTransformSlotAndInstanceId & 0xFFFFFF00) | (V & 0xFF); }
inline void SetParentInstanceId(FResolveAttachmentBlockHeader& H, uint32 V)     { H.ParentCurrentTransformSlotAndInstanceId = (H.ParentCurrentTransformSlotAndInstanceId & 0x000000FF) | (V << 8); }
inline uint32 GetMaxTransformCount(const FResolveAttachmentBlockHeader& H)    { return H.ParentMaxTransformCountAndBoneIndex & 0xFFFF; }
inline uint32 GetBoneIndex(const FResolveAttachmentBlockHeader& H)            { return H.ParentMaxTransformCountAndBoneIndex >> 16; }
inline uint32 GetCurrentTransformSlot(const FResolveAttachmentBlockHeader& H) { return H.ParentCurrentTransformSlotAndInstanceId & 0xFF; }
inline uint32 GetParentInstanceId(const FResolveAttachmentBlockHeader& H)     { return H.ParentCurrentTransformSlotAndInstanceId >> 8; }
#else
uint GetMaxTransformCount(FResolveAttachmentBlockHeader H)    { return H.ParentMaxTransformCountAndBoneIndex & 0xFFFF; }
uint GetBoneIndex(FResolveAttachmentBlockHeader H)            { return H.ParentMaxTransformCountAndBoneIndex >> 16; }
uint GetCurrentTransformSlot(FResolveAttachmentBlockHeader H) { return H.ParentCurrentTransformSlotAndInstanceId & 0xFF; }
uint GetParentInstanceId(FResolveAttachmentBlockHeader H)     { return H.ParentCurrentTransformSlotAndInstanceId >> 8; }
#endif

#define ANIM_BANK_FLAG_NONE			0x0
#define ANIM_BANK_FLAG_LOOPING		0x1
#define ANIM_BANK_FLAG_AUTOSTART	0x2

#ifdef __cplusplus
} // namespace UE::HLSL
#endif

#if defined(__cplusplus)
#define UINT_TYPE unsigned int
#else
#define UINT_TYPE uint
#endif

// Animation is always sampled at 30hz
#define ANIM_BANK_SAMPLE_RATE 30.0f

#define SKINNING_BUFFER_HIERARCHY_OFFSET_BITS 22
#define SKINNING_BUFFER_HIERARCHY_OFFSET_MAX (1 << SKINNING_BUFFER_HIERARCHY_OFFSET_BITS) - 1

#define SKINNING_BUFFER_TRANSFORM_OFFSET_BITS 22
#define SKINNING_BUFFER_TRANSFORM_OFFSET_MAX (1 << SKINNING_BUFFER_TRANSFORM_OFFSET_BITS) - 1

#define SKINNING_BUFFER_OBJECT_SPACE_OFFSET_BITS 22
#define SKINNING_BUFFER_OBJECT_SPACE_OFFSET_MAX (1 << SKINNING_BUFFER_OBJECT_SPACE_OFFSET_BITS) - 1

#define SKINNING_BUFFER_INFLUENCE_BITS 5
#define SKINNING_BUFFER_INFLUENCE_MAX (1 << SKINNING_BUFFER_INFLUENCE_BITS)

struct FSkinningHeader
{
	UINT_TYPE HierarchyBufferOffset   : SKINNING_BUFFER_HIERARCHY_OFFSET_BITS;
	UINT_TYPE TransformBufferOffset   : SKINNING_BUFFER_TRANSFORM_OFFSET_BITS;
	UINT_TYPE ObjectSpaceBufferOffset : SKINNING_BUFFER_OBJECT_SPACE_OFFSET_BITS;
	UINT_TYPE MaxTransformCount       : 16;
	UINT_TYPE CurrentTransformSlot    : 1;
	UINT_TYPE MaxInfluenceCount       : SKINNING_BUFFER_INFLUENCE_BITS;
	UINT_TYPE bHasScale               : 1;
	UINT_TYPE bIsRefPose              : 1;
	UINT_TYPE Unused                  : 6;
};

#endif // defined(__cplusplus) || COMPILER_SUPPORTS_HLSL2021

#if USE_COMPRESSED_BONE_TRANSFORM
struct FCompressedBoneTransform
{
	UINT_TYPE Data[8];
};
#else
#if defined(__cplusplus)
#define FCompressedBoneTransform FMatrix3x4
#else
#define FCompressedBoneTransform float3x4
#endif
#endif

#if USE_COMPRESSED_BONE_TRANSFORM

#define COMPRESSED_BONE_TRANSFORM_PIXEL_FORMAT PF_Unknown 
#define FPersistentCompressedBoneTransformBuffer TPersistentByteAddressBuffer<FCompressedBoneTransform>
#define FCompressedBoneTransformBufferScatterUploader TByteAddressBufferScatterUploader<FCompressedBoneTransform>
#define FCompressedBoneTransformBuffer ByteAddressBuffer
#define FCompressedBoneTransformRWBuffer RWByteAddressBuffer

#define SHADER_PARAMETER_COMPRESSED_BONE_TRANSFORM_SRV(Name) SHADER_PARAMETER_SRV(ByteAddressBuffer, Name)
#define SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_SRV(Name) SHADER_PARAMETER_RDG_BUFFER_SRV(ByteAddressBuffer, Name)
#define SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_UAV(Name) SHADER_PARAMETER_RDG_BUFFER_UAV(RWByteAddressBuffer, Name)

#else

#define COMPRESSED_BONE_TRANSFORM_PIXEL_FORMAT PF_A32B32G32R32F
#define FPersistentCompressedBoneTransformBuffer TPersistentFloat4Buffer<FCompressedBoneTransform>
#define FCompressedBoneTransformBufferScatterUploader TFloat4BufferScatterUploader<FCompressedBoneTransform>
#define FCompressedBoneTransformBuffer Buffer<float4>
#define FCompressedBoneTransformRWBuffer RWBuffer<float4>

#define SHADER_PARAMETER_COMPRESSED_BONE_TRANSFORM_SRV(Name) SHADER_PARAMETER_SRV(Buffer<float4>, Name)
#define SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_SRV(Name) SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float4>, Name)
#define SHADER_PARAMETER_RDG_COMPRESSED_BONE_TRANSFORM_UAV(Name) SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float4>, Name)

#endif

#ifdef __cplusplus

#include "Matrix3x4.h"

#define REF_POSE_TRANSFORM_PROVIDER_GUID 0x665207E7, 0x449A4FB1, 0xA298F7AD, 0x8F989B11
#define ANIM_BANK_GPU_TRANSFORM_PROVIDER_GUID 0xA5C0027A, 0x8F884C7C, 0x9312F138, 0x71A9300F
#define ANIM_BANK_CPU_TRANSFORM_PROVIDER_GUID 0xE7D6173D, 0x246F431A, 0x912D384E, 0x156C0D2C
#define ANIM_SEQUENCE_GPU_TRANSFORM_PROVIDER_GUID 0x5CFA81C1, 0xF3D8486B, 0x81FF23D3, 0x72EB523F
#define ANIM_RUNTIME_TRANSFORM_PROVIDER_GUID 0xF1508490, 0xFCC24BB9, 0xA9F277B3, 0x1AF766F0
#define ANIM_MESH_OBJECT_TRANSFORM_PROVIDER_GUID 0x1073C0C5, 0xB981F6FD, 0xBF41B1C2, 0xC4BA0C50

inline void StoreCompressedBoneTransform(FCompressedBoneTransform& CompressedTransform, const FMatrix44f& Transform)
{
#if USE_COMPRESSED_BONE_TRANSFORM
	//TODO: Optimize
	*(FVector3f*)&CompressedTransform.Data[0] = Transform.GetOrigin();

	float Tmp[8] = { Transform.M[0][0], Transform.M[0][1], Transform.M[0][2], Transform.M[1][0],
					 Transform.M[1][1], Transform.M[1][2], Transform.M[2][0], Transform.M[2][1] };

	uint16* Ptr = (uint16*)&CompressedTransform.Data[3];

	FPlatformMath::VectorStoreHalf(&Ptr[0], &Tmp[0]);
	FPlatformMath::VectorStoreHalf(&Ptr[4], &Tmp[4]);
	FPlatformMath::StoreHalf(&Ptr[8], Transform.M[2][2]);
	Ptr[9] = 0;
#else
	Transform.To3x4MatrixTranspose((float*)&CompressedTransform);
#endif
}

inline void StoreCompressedBoneTransform(FCompressedBoneTransform* CompressedTransform, const FMatrix44f& Transform)
{
	StoreCompressedBoneTransform(*CompressedTransform, Transform);
}

inline void StoreCompressedBoneTransform(FCompressedBoneTransform& CompressedTransform, const FMatrix3x4& Transform)
{
#if USE_COMPRESSED_BONE_TRANSFORM
	*(FVector3f*)&CompressedTransform.Data[0] = FVector3f(Transform.M[0][3], Transform.M[1][3], Transform.M[2][3]);

	float Tmp[8] = { Transform.M[0][0], Transform.M[1][0], Transform.M[2][0], Transform.M[0][1],
					 Transform.M[1][1], Transform.M[2][1], Transform.M[0][2], Transform.M[1][2] };

	uint16* Ptr = (uint16*)&CompressedTransform.Data[3];

	FPlatformMath::VectorStoreHalf(&Ptr[0], &Tmp[0]);
	FPlatformMath::VectorStoreHalf(&Ptr[4], &Tmp[4]);
	FPlatformMath::StoreHalf(&Ptr[8], Transform.M[2][2]);
	Ptr[9] = 0;
#else
	CompressedTransform = Transform;
#endif
}

inline void SetCompressedBoneTransformIdentity(FCompressedBoneTransform& Transform)
{
#if USE_COMPRESSED_BONE_TRANSFORM
	Transform.Data[0] = 0;			// Origin.X = 0
	Transform.Data[1] = 0;			// Origin.Y = 0
	Transform.Data[2] = 0;			// Origin.Z = 0
	Transform.Data[3] = 0x3C00u;	// XAxis.X = 1, XAxis.Y = 0
	Transform.Data[4] = 0;			// YAxis.Z = 0, YAxis.X = 0
	Transform.Data[5] = 0x3C00u;	// YAxis.Y = 1, YAxis.Z = 0
	Transform.Data[6] = 0;			// ZAxis.X = 0, ZAxis.Y = 0
	Transform.Data[7] = 0x3C00u;	// ZAxis.Z = 1
#else
	Transform.SetIdentity();
#endif
}

#endif
