// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// The packed size of the spline mesh data in float4s
#define SPLINE_MESH_PARAMS_FLOAT4_SIZE 7

// Definitions for sizing the scene spline mesh texture and encoding spline addresses
#define SPLINE_MESH_TEXEL_WIDTH_BITS		(6u)
#define SPLINE_MESH_TEXEL_WIDTH				(1u << SPLINE_MESH_TEXEL_WIDTH_BITS)
#define SPLINE_MESH_TEXEL_WIDTH_MASK		(SPLINE_MESH_TEXEL_WIDTH - 1u)
#define SPLINE_MESH_TEXTURE_MAX_DIMENSION	(16 * 1024)

#ifdef __cplusplus
#include "HLSLTypeAliases.h"

namespace SplineMeshInternal::ShaderParams::HLSL
{
	// The struct below doesn't need to be binary compatible between C++ and HLSL because of custom packing/unpacking methods.
	// So we'll let "half" just mean float on the C++ side in this particular context.
	// (NOTE: placed in separate namespace so as not to pollute UE::HLSL)
	using namespace UE::HLSL;
	using half = float;
	using half2 = float2;
	using half3 = float3;
#endif // __cplusplus

struct FSplineMeshShaderParams
{
	float3 StartPos;
	float3 EndPos;
	float3 StartTangent;
	float3 EndTangent;
	float2 StartOffset;
	float2 EndOffset;
	half StartRoll;
	half EndRoll;
	half2 StartScale;
	half2 EndScale;
	float MeshZScale;
	float MeshZOffset;
	float2 MeshDeformScaleMinMax;
	bool bSmoothInterpRollScale;
	half3 SplineUpDir;
	half3 MeshDir;
	half3 MeshX;
	half3 MeshY;
	uint2 TextureCoord;
	half SplineDistToTexelScale;
	half SplineDistToTexelOffset;
	half NaniteClusterBoundsScale;
};

#ifdef __cplusplus
} // namespace SplineMeshInternal::ShaderParams::HLSL
using FSplineMeshShaderParams = SplineMeshInternal::ShaderParams::HLSL::FSplineMeshShaderParams;
#endif // __cplusplus

#ifndef __cplusplus
#include "/Engine/Private/Quaternion.ush"

// HLSL unpack method
FSplineMeshShaderParams UnpackSplineMeshParams(float4 PackedParams[SPLINE_MESH_PARAMS_FLOAT4_SIZE])
{
	FSplineMeshShaderParams Output;
	Output.StartPos 				= PackedParams[0].xyz;
	Output.EndPos 					= PackedParams[1].xyz;
	Output.StartTangent 			= PackedParams[2].xyz;
	Output.EndTangent 				= float3(PackedParams[0].w, PackedParams[1].w, PackedParams[2].w);
	Output.StartOffset 				= PackedParams[3].xy;
	Output.EndOffset 				= PackedParams[3].zw;
	Output.StartRoll 				= f16tof32(asuint(PackedParams[4].x));
	Output.EndRoll 					= f16tof32(asuint(PackedParams[4].x) >> 16u);
	Output.StartScale 				= half2(f16tof32(asuint(PackedParams[4].y)),
											f16tof32(asuint(PackedParams[4].y) >> 16u));
	Output.EndScale 				= half2(f16tof32(asuint(PackedParams[4].z)),
											f16tof32(asuint(PackedParams[4].z) >> 16u));
	Output.TextureCoord				= uint2(asuint(PackedParams[4].w) & 0xFFFFu,
											asuint(PackedParams[4].w) >> 16u);
	Output.MeshZScale 				= PackedParams[5].x;
	Output.MeshZOffset 				= PackedParams[5].y;
	Output.MeshDeformScaleMinMax	= half2(f16tof32(asuint(PackedParams[5].z)),
											f16tof32(asuint(PackedParams[5].z) >> 16u));
	Output.SplineDistToTexelScale	= f16tof32(asuint(PackedParams[5].w));
	Output.SplineDistToTexelOffset	= f16tof32(asuint(PackedParams[5].w) >> 16u);
	Output.SplineUpDir 				= half3(SNorm16ToF32(asuint(PackedParams[6].x)),
											SNorm16ToF32(asuint(PackedParams[6].x) >> 16u),
											SNorm16ToF32(asuint(PackedParams[6].y)));
	Output.NaniteClusterBoundsScale	= f16tof32((asuint(PackedParams[6].y) >> 16u) & 0x7FFFu);
	Output.bSmoothInterpRollScale	= (asuint(PackedParams[6].y) >> 31u) != 0;
	FQuat MeshQuat					= FQuat(SNorm16ToF32(asuint(PackedParams[6].z)),
											SNorm16ToF32(asuint(PackedParams[6].z) >> 16u),
											SNorm16ToF32(asuint(PackedParams[6].w)),
											SNorm16ToF32(asuint(PackedParams[6].w) >> 16u));
	half3x3 MeshRot 				= QuatToMatrix(MeshQuat);
	Output.MeshDir					= MeshRot[0];
	Output.MeshX					= MeshRot[1];
	Output.MeshY					= MeshRot[2];

	return Output;
}
#endif // !__cplusplus
