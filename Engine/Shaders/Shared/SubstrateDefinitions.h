// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
	SubstrateDefinitions.ush: used in ray tracing shaders and C++ code to define common constants
	!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#pragma once

#ifndef __cplusplus
// Change this to force recompilation of all Substrate dependent shaders (for instance https://guidgenerator.com/online-guid-generator.aspx)
#pragma message("UESHADERMETADATA_VERSION 96CDF7E5-2EB8-4E48-8C1E-23B557A7B5BB")
#endif

// Closure offsets are packed into 32bits, each entry using SUBSTRATE_CLOSURE_OFFSET_BIT_COUNT bits
#define SUBSTRATE_MAX_CLOSURE_COUNT_FOR_CLOSUREOFFSET	8u
#define SUBSTRATE_CLOSURE_OFFSET_BIT_COUNT				4u
#define SUBSTRATE_CLOSURE_OFFSET_BIT_MASK				0xF

// We can only ever use SUBSTRATE_MAX_CLOSURE_COUNT_FOR_CLOSUREOFFSET for Lumen, so we use that as a global closure count limit today.
#define SUBSTRATE_MAX_CLOSURE_COUNT						SUBSTRATE_MAX_CLOSURE_COUNT_FOR_CLOSUREOFFSET
#define SUBSTRATE_MAX_OPERATOR_COUNT					15

// It this is changed, STATE_BIT_COUNT_SHAREDLOCALBASESID and HEADER_BIT_COUNT_SHAREDLOCALBASES_COUNT also needs to be updated
#define SUBSTRATE_MAX_SHAREDLOCALBASES_REGISTERS		4

#define SUBSTRATE_PACKED_SHAREDLOCALBASIS_STRIDE_BYTES	4

// As of today, a fully simplified material is a slab with all features allowed. It can thus be complex if anisotropy is enabled and in this case eats up to 32bytes.
// SUBSTRATE_TODO: fully simplified should remove all features but fuzz maybe. 
#define SUBSTRATE_FULLY_SIMPLIFIED_NUM_UINTS			(32/4)

#define SUBSTRATE_BSDF_TYPE_SLAB						0
#define SUBSTRATE_BSDF_TYPE_VOLUMETRICFOGCLOUD			1
#define SUBSTRATE_BSDF_TYPE_UNLIT						2
#define SUBSTRATE_BSDF_TYPE_HAIR						3
#define SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER			4
#define SUBSTRATE_BSDF_TYPE_EYE							5
#define SUBSTRATE_BSDF_TYPE_TOON						6
//define SUBSTRATE_BSDF_TYPE_CUSTOM						7 Reserved for Custom shading model
// When more than 7 BSDF must exists, please update STATE_BIT_COUNT_BSDF and FSubstrateClassification.ShadingModels packing in Substrate.ush

// The size of Substrate material classification tiles on screen
#define SUBSTRATE_TILE_SIZE								8
#define SUBSTRATE_TILE_SIZE_DIV_AS_SHIFT				3

// The default thickness of a layer is considered to be 0.01 centimeter = 0.1 millimeter
#define SUBSTRATE_LAYER_DEFAULT_THICKNESS_CM			0.01f

// Min GGX/Fuzz Roughness to avoid numerical issue
// Avoid INF value when computing GGX's D term (similar to IntegrateBxDF() logic)
#define SUBSTRATE_MIN_GGX_ROUGHNESS						0.02f
#define SUBSTRATE_MIN_FUZZ_ROUGHNESS					0.02f

#define SUBSTRATE_BASE_PASS_MRT_OUTPUT_COUNT			3

#define SUBSTRATE_SSS_DATA_UINT_COUNT					2

#define SUBSTRATE_OPERATOR_WEIGHT						0
#define SUBSTRATE_OPERATOR_VERTICAL						1
#define SUBSTRATE_OPERATOR_HORIZONTAL					2
#define SUBSTRATE_OPERATOR_ADD							3
#define SUBSTRATE_OPERATOR_SELECT						4
#define SUBSTRATE_OPERATOR_BSDF							5
#define SUBSTRATE_OPERATOR_BSDF_LEGACY					6

// Sometimes, Unlit or Weight operators are used to transport data for other meaning (e.g. Light Function or ConvertToDecal)
#define SUBSTRATE_OPERATOR_SUBUSAGE_NONE				0
#define SUBSTRATE_OPERATOR_SUBUSAGE_UI					1
#define SUBSTRATE_OPERATOR_SUBUSAGE_LIGHTFUNCTION		2
#define SUBSTRATE_OPERATOR_SUBUSAGE_POSTPROCESS			3
#define SUBSTRATE_OPERATOR_SUBUSAGE_DECAL				4

// This must map directly to ESubstrateTileMaterialType
#define SUBSTRATE_TILE_TYPE_SIMPLE						0
#define SUBSTRATE_TILE_TYPE_SINGLE						1
#define SUBSTRATE_TILE_TYPE_COMPLEX						2
#define SUBSTRATE_TILE_TYPE_COMPLEX_SPECIAL				3
#define SUBSTRATE_TILE_TYPE_ROUGH_REFRACT				4
#define SUBSTRATE_TILE_TYPE_ROUGH_REFRACT_SSS_WITHOUT	5
#define SUBSTRATE_TILE_TYPE_DECAL_SIMPLE				6
#define SUBSTRATE_TILE_TYPE_DECAL_SINGLE				7
#define SUBSTRATE_TILE_TYPE_DECAL_COMPLEX				8
#define SUBSTRATE_TILE_TYPE_COUNT						9

// Tile bitmasked used within the StochasticLighting classification
#define SUBSTRATE_TILE_BITMASK_SIMPLE					(1u<<SUBSTRATE_TILE_TYPE_SIMPLE)
#define SUBSTRATE_TILE_BITMASK_SINGLE					(1u<<SUBSTRATE_TILE_TYPE_SINGLE)
#define SUBSTRATE_TILE_BITMASK_COMPLEX					(1u<<SUBSTRATE_TILE_TYPE_COMPLEX)
#define SUBSTRATE_TILE_BITMASK_COMPLEXSPECIAL			(1u<<SUBSTRATE_TILE_TYPE_COMPLEX_SPECIAL)
#define SUBSTRATE_TILE_BITMASK_ROUGH_REFRACT			(1u<<SUBSTRATE_TILE_TYPE_ROUGH_REFRACT)
#define SUBSTRATE_TILE_BITMASK_ROUGH_REFRACT_SSS_WITHOUT	(1u<<SUBSTRATE_TILE_TYPE_ROUGH_REFRACT_SSS_WITHOUT)
#define SUBSTRATE_TILE_BITMASK_DECAL_SIMPLE				(1u<<SUBSTRATE_TILE_TYPE_DECAL_SIMPLE)
#define SUBSTRATE_TILE_BITMASK_DECAL_SINGLE				(1u<<SUBSTRATE_TILE_TYPE_DECAL_SINGLE)
#define SUBSTRATE_TILE_BITMASK_DECAL_COMPLEX			(1u<<SUBSTRATE_TILE_TYPE_DECAL_COMPLEX)
#define SUBSTRATE_TILE_BITMASK_DECAL					(SUBSTRATE_TILE_BITMASK_DECAL_SIMPLE|SUBSTRATE_TILE_BITMASK_DECAL_SINGLE|SUBSTRATE_TILE_BITMASK_DECAL_COMPLEX)


#define SUBSTRATE_MATERIAL_TYPE_SIMPLE					0
#define SUBSTRATE_MATERIAL_TYPE_SINGLE					1
#define SUBSTRATE_MATERIAL_TYPE_COMPLEX					2
#define SUBSTRATE_MATERIAL_TYPE_COMPLEX_SPECIAL			3
#define SUBSTRATE_MATERIAL_TYPE_COUNT					4

#define SUBSTRATE_TILE_ENCODING_16BITS 					0
#define SUBSTRATE_TILE_ENCODING_8BITS  					1

// Specular profile
#define SUBSTRATE_SPECULAR_PROFILE_ENTRY_LIGHT			0u
#define SUBSTRATE_SPECULAR_PROFILE_ENTRY_ENV			1u
#define SUBSTRATE_SPECULAR_PROFILE_ENTRY_COUNT			2u

// sizeof(FRHIDrawIndirectParameters) = 4 uints = 16 bytes
#define GetSubstrateTileTypeDrawIndirectArgOffset_Byte(x)  (x * 16)
#define GetSubstrateTileTypeDrawIndirectArgOffset_DWord(x) (x * 4)

// sizeof(FRHIDispatchIndirectParameters) can vary per-platform
#ifdef __cplusplus
	#define GetSubstrateTileTypeDispatchIndirectArgOffset_Byte(x)  (x * sizeof(FRHIDispatchIndirectParameters))
	#define GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(x) (x * sizeof(FRHIDispatchIndirectParameters) / sizeof(uint32))
#else
	#define GetSubstrateTileTypeDispatchIndirectArgOffset_Byte(x)  (x * DISPATCH_INDIRECT_UINT_COUNT * 4)
	#define GetSubstrateTileTypeDispatchIndirectArgOffset_DWord(x) (x * DISPATCH_INDIRECT_UINT_COUNT)
#endif

#define HEADER_MATERIALMODE_NONE				0
#define HEADER_MATERIALMODE_SLAB_SIMPLE			1
#define HEADER_MATERIALMODE_SLAB_SINGLE			2
#define HEADER_MATERIALMODE_SLAB_COMPLEX		3
#define HEADER_MATERIALMODE_SLWATER				4
#define HEADER_MATERIALMODE_HAIR				5
#define HEADER_MATERIALMODE_EYE					6
// Up to 8 material modes: 1 remains.

// Special optmised modes for the single BSDF path.
#define HEADER_SINGLE_OPTLEGACYMODE_BIT_COUNT	3
#define HEADER_SINGLE_OPTLEGACYMODE_BIT_MASK	((1u<<HEADER_SINGLE_OPTLEGACYMODE_BIT_COUNT)-1u)
#define SINGLE_OPTLEGACYMODE_NONE				0
#define SINGLE_OPTLEGACYMODE_CLEARCOAT			1
#define SINGLE_OPTLEGACYMODE_CLOTH				2
#define SINGLE_OPTLEGACYMODE_SSSWRAP			3
#define SINGLE_OPTLEGACYMODE_SSSPROFILE			4
#define SINGLE_OPTLEGACYMODE_TWO_SIDED_SSSWRAP	5
// 5 more remains: Also how to deal with Aniso?

// Sub-surface scattering types
#define SSS_TYPE_NONE							(0u)
#define SSS_TYPE_WRAP							(1u)
#define SSS_TYPE_TWO_SIDED_WRAP					(2u)
#define SSS_TYPE_DIFFUSION						(3u)
#define SSS_TYPE_DIFFUSION_PROFILE				(4u)
#define SSS_TYPE_SIMPLEVOLUME					(5u)
#define SSS_TYPE_TRANSMITTANCE_SHADOW			(6u)
#define SSS_TYPE_COUNT							(7u)
