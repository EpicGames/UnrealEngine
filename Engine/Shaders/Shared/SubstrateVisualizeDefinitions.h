// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
	SubstrateVisualise.h: used in ray tracing shaders and C++ code to define common constants
	!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#pragma once

#if defined(__cplusplus) || SUBSTRATE_ENABLED

#ifndef __cplusplus
// Change this to force recompilation of all Substrate dependent shaders (for instance https://guidgenerator.com/online-guid-generator.aspx)
#pragma message("UESHADERMETADATA_VERSION 24677C38-A71C-49D5-B60A-DFF59F57B39D")
#include "/Engine/Shared/SubstrateDefinitions.h"

#if PS4_PROFILE || FEATURE_LEVEL <= FEATURE_LEVEL_SM5
// For platform having trouble avoiding register spilling.
// To fix this in general, we will have to break down the FSubstratePixelDebugData structure into Header and Closure structures with interleaved write to buffer.
#define SUBSTRATE_VIS_CLOSURE_COUNT 1
#else
#define SUBSTRATE_VIS_CLOSURE_COUNT SUBSTRATE_MAX_CLOSURE_COUNT
#endif

#else
#include "SubstrateDefinitions.h"
#define SUBSTRATE_VIS_CLOSURE_COUNT SUBSTRATE_MAX_CLOSURE_COUNT
#endif

struct FSubsterateDebugClosure
{
	int Type;

	int bHasWeightL;// = BSDF_GETHASTRANSABOVE(BSDF);
	int bHasGreyWeightV;// = BSDF_GETHASGREYWEIGHT_V(BSDF);
	int Address; // SubstrateAddressing.CurrentIndex

	int NormalID; //BSDF_GETSHAREDLOCALBASISID(BSDF)
	int BasisType; // SubstrateGetSharedLocalBasisType(HEADER_GETSHAREDLOCALBASESTYPE(Header.PackedHeader), BSDF_GETSHAREDLOCALBASISID(BSDF));

	//////////
	// Slabs
	int bIsTopLayer; // BSDF_GETISTOPLAYER(BSDF)
	int SSSType; // BSDF_GETSSSTYPE(BSDF) != SSS_TYPE_NONE
	int bIsThin; // BSDF_GETISTHIN(BSDF)
	float LuminanceWeightR, LuminanceWeightG, LuminanceWeightB; // bHasGreyWeightV BSDF.LuminanceWeightV.x or rgb
	float TransmittanceAboveAlongNR, TransmittanceAboveAlongNG, TransmittanceAboveAlongNB; // bHasGreyTopTrans BSDF.TransmittanceAboveAlongN.x or rgb
	float CoverageAboveAlongN;

	float DiffuseR,	DiffuseG,	DiffuseB;
	float F0R,		F0G,		F0B;
	float Roughness;

	int bHasF90;
	float F90R, F90G, F90B;

	int bHasAnisotropy;
	float Anisotropy;

	int bHasHaziness;
	float HazeRoughness;
	float HazeWeight;
	int HazeSimpleClearCoatMode;
	int HasBottomNormal;

	float  SSSOpacity;
	float SSSMFPR, SSSMFPG, SSSMFPB;
	float SSSRescaledMFPR, SSSRescaledMFPG, SSSRescaledMFPB;
	float  SSSPhase;
	float  SSSThickness;
	float  SSSProfileRadius;
	int SSSPRofileID;

	float FuzzAmount;
	float FuzzColorR, FuzzColorG, FuzzColorB;
	float FuzzRoughness;

	float GlintValue;
	float GlintUVDDXx;
	float GlintUVDDXy;
	float GlintUVDDYx;
	float GlintUVDDYy;

	int SpecProfileID;
	int SpecProfileParameterization;
};

struct FSubstratePixelDebugData
{
	int ClosureCount;
	int MaterialMode;
	int bIsComplexSpecialMaterial;
	int OptimisedLegacyMode;

	float MaterialAO;
	float IndirectIrradiance;
	float TopLayerRoughness;
	int HasPrecShadowMask;
	int HasZeroPrecShadowMask;
	int DoesCastContactShadow;
	int HasDynamicIndirectShadowCasterRepresentation;
	int HasSubsurface;
	int LocalBasesCount;

	FSubsterateDebugClosure Closures[SUBSTRATE_VIS_CLOSURE_COUNT];

	int MemoryDisplayMode;
	int MemorySlotA;
	int MemorySlotB;
	int MemorySlotC;
	int MemorySSSData;
	int MemoryTotal;

	int GPUFrameNumber;
};

#ifndef __cplusplus
uint SubstrateDebugDataSizeInUints;
RWStructuredBuffer<int> SubstrateDebugDataUAV;
struct FSubstrateDebugDataSerializer
{
	int WriteIndex;

	void Serialize(int Data)
	{
		if (WriteIndex < SubstrateDebugDataSizeInUints)
		{
			SubstrateDebugDataUAV[WriteIndex++] = Data;
		}
	}

	void Serialize(float Data)
	{
		if (WriteIndex < SubstrateDebugDataSizeInUints)
		{
			SubstrateDebugDataUAV[WriteIndex++] = asuint(Data);
		}
	}
};
#else
struct FSubstrateDebugDataSerializer
{
	int ReadIndex = 0;
	int* SubstratePixelDebugData = nullptr;

	void Serialize(int& Data)
	{
		Data = SubstratePixelDebugData[ReadIndex++];
	}

	void Serialize(float& Data)
	{
		Data = ((float*)SubstratePixelDebugData)[ReadIndex++];
	}
};
#endif

#ifndef __cplusplus
void SerializeSubstratePixelDebugData(in FSubstrateDebugDataSerializer S, in FSubstratePixelDebugData D)
#else
void SerializeSubstratePixelDebugData(FSubstrateDebugDataSerializer& S, FSubstratePixelDebugData& D)
#endif
{
	S.Serialize(D.ClosureCount);
#ifdef __cplusplus
	// Safe guard to avoid crash in case of readback problem.
	D.ClosureCount = FMath::Min(D.ClosureCount, int(SUBSTRATE_VIS_CLOSURE_COUNT));
#else
	D.ClosureCount = min(D.ClosureCount, int(SUBSTRATE_VIS_CLOSURE_COUNT));
#endif

	S.Serialize(D.MaterialMode);
	S.Serialize(D.bIsComplexSpecialMaterial);
	S.Serialize(D.OptimisedLegacyMode);

	S.Serialize(D.MaterialAO);
	S.Serialize(D.IndirectIrradiance);
	S.Serialize(D.TopLayerRoughness);
	S.Serialize(D.HasPrecShadowMask);
	S.Serialize(D.HasZeroPrecShadowMask);
	S.Serialize(D.DoesCastContactShadow);
	S.Serialize(D.HasDynamicIndirectShadowCasterRepresentation);
	S.Serialize(D.HasSubsurface);
	S.Serialize(D.LocalBasesCount);

	for (int i = 0; i < D.ClosureCount; ++i)
	{
		S.Serialize(D.Closures[i].Type);

		S.Serialize(D.Closures[i].bHasWeightL);
		S.Serialize(D.Closures[i].bHasGreyWeightV);
		S.Serialize(D.Closures[i].Address);

		S.Serialize(D.Closures[i].NormalID);
		S.Serialize(D.Closures[i].BasisType);

		S.Serialize(D.Closures[i].bIsTopLayer);
		S.Serialize(D.Closures[i].SSSType);
		S.Serialize(D.Closures[i].bIsThin);
		S.Serialize(D.Closures[i].LuminanceWeightR);
		S.Serialize(D.Closures[i].LuminanceWeightG);
		S.Serialize(D.Closures[i].LuminanceWeightB);
		S.Serialize(D.Closures[i].TransmittanceAboveAlongNR);
		S.Serialize(D.Closures[i].TransmittanceAboveAlongNG);
		S.Serialize(D.Closures[i].TransmittanceAboveAlongNB);
		S.Serialize(D.Closures[i].CoverageAboveAlongN);

		S.Serialize(D.Closures[i].DiffuseR);
		S.Serialize(D.Closures[i].DiffuseG);
		S.Serialize(D.Closures[i].DiffuseB);
		S.Serialize(D.Closures[i].F0R);
		S.Serialize(D.Closures[i].F0G);
		S.Serialize(D.Closures[i].F0B);
		S.Serialize(D.Closures[i].Roughness);

		S.Serialize(D.Closures[i].bHasF90);
		S.Serialize(D.Closures[i].F90R);
		S.Serialize(D.Closures[i].F90G);
		S.Serialize(D.Closures[i].F90B);

		S.Serialize(D.Closures[i].bHasAnisotropy);
		S.Serialize(D.Closures[i].Anisotropy);

		S.Serialize(D.Closures[i].bHasHaziness);
		S.Serialize(D.Closures[i].HazeRoughness);
		S.Serialize(D.Closures[i].HazeWeight);
		S.Serialize(D.Closures[i].HazeSimpleClearCoatMode);
		S.Serialize(D.Closures[i].HasBottomNormal);

		S.Serialize(D.Closures[i].SSSOpacity);
		S.Serialize(D.Closures[i].SSSMFPR);
		S.Serialize(D.Closures[i].SSSMFPG);
		S.Serialize(D.Closures[i].SSSMFPB);
		S.Serialize(D.Closures[i].SSSRescaledMFPR);
		S.Serialize(D.Closures[i].SSSRescaledMFPG);
		S.Serialize(D.Closures[i].SSSRescaledMFPB);
		S.Serialize(D.Closures[i].SSSPhase);
		S.Serialize(D.Closures[i].SSSThickness);
		S.Serialize(D.Closures[i].SSSProfileRadius);
		S.Serialize(D.Closures[i].SSSPRofileID);

		S.Serialize(D.Closures[i].FuzzAmount);
		S.Serialize(D.Closures[i].FuzzColorR);
		S.Serialize(D.Closures[i].FuzzColorG);
		S.Serialize(D.Closures[i].FuzzColorB);
		S.Serialize(D.Closures[i].FuzzRoughness);

		S.Serialize(D.Closures[i].GlintValue);
		S.Serialize(D.Closures[i].GlintUVDDXx);
		S.Serialize(D.Closures[i].GlintUVDDXy);
		S.Serialize(D.Closures[i].GlintUVDDYx);
		S.Serialize(D.Closures[i].GlintUVDDYy);

		S.Serialize(D.Closures[i].SpecProfileID);
		S.Serialize(D.Closures[i].SpecProfileParameterization);
	}

	S.Serialize(D.MemoryDisplayMode);
	S.Serialize(D.MemorySlotA);
	S.Serialize(D.MemorySlotB);
	S.Serialize(D.MemorySlotC);
	S.Serialize(D.MemorySSSData);
	S.Serialize(D.MemoryTotal);

	S.Serialize(D.GPUFrameNumber);
}

#ifndef __cplusplus


#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE && SUBSTRATE_GBUFFER_FORMAT == 0

#include "/Engine/Private/Substrate/SubstrateRead.ush"
#include "/Engine/Private/Substrate/SubstrateSubsurface.ush"

FSubstratePixelHeader UnpackSubstrateHeaderIn(inout FSubstrateAddressing SubstrateAddressing)
{	
	const FGBufferData GBufferData = GetScreenSpaceDataUint(SubstrateAddressing.PixelCoords).GBuffer;

	FSubstratePixelHeader Out = InitialiseSubstratePixelHeader();
	if (GBufferData.ShadingModelID != SHADINGMODELID_UNLIT)
	{
		uint Mode = HEADER_MATERIALMODE_NONE;
		if (GBufferData.ShadingModelID == SHADINGMODELID_SINGLELAYERWATER)
		{
			Mode = HEADER_MATERIALMODE_SLWATER;
		}
		else if (GBufferData.ShadingModelID == SHADINGMODELID_HAIR)
		{
			Mode = HEADER_MATERIALMODE_HAIR;
		}
		else if (GBufferData.ShadingModelID == SHADINGMODELID_EYE)
		{
			Mode = HEADER_MATERIALMODE_EYE;
		}
		else
		{
			const bool bHasAnisotropy = abs(GBufferData.Anisotropy) > 0;
			const bool bComplexMaterial = bHasAnisotropy;
			const bool bSimpleMaterial = !bComplexMaterial && GBufferData.ShadingModelID == SHADINGMODELID_DEFAULT_LIT;
			const bool bSingleMaterial = !bComplexMaterial && !bSimpleMaterial;
			const bool bScreenSpaceSubsurfaceScattering = GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE || GBufferData.ShadingModelID == SHADINGMODELID_EYE;

			Mode = bSimpleMaterial ? HEADER_MATERIALMODE_SLAB_SIMPLE : bSingleMaterial ? HEADER_MATERIALMODE_SLAB_SINGLE : HEADER_MATERIALMODE_SLAB_COMPLEX;
		}

		const bool bHasSSS = 
		   GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE
		|| GBufferData.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN		
		|| GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE
		|| GBufferData.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE
		|| GBufferData.ShadingModelID == SHADINGMODELID_EYE;

		Out.ClosureCount = 1; 
		Out.SetMaterialMode(Mode);
		Out.SetHasSubsurface(bHasSSS);
		Out.SetHasPrecShadowMask(HasPrecShadowMask(GBufferData));
		Out.SetZeroPrecShadowMask(HasZeroPrecShadowMask(GBufferData));
		Out.SetIsFirstPerson(IsFirstPerson(GBufferData));
		Out.SetCastContactShadow(CastContactShadow(GBufferData));
		Out.SetDynamicIndirectShadowCasterRepresentation(HasDynamicIndirectShadowCasterRepresentation(GBufferData));

		#if SUBSTRATE_INLINE_SHADING
		Out.IrradianceAO = SubstrateGetIrradianceAndAO(GBufferData);
		#endif
	}

	SubstrateAddressing.CurrentIndex = 0;
	SubstrateAddressing.ReadBytes = 0;
	return Out;
}

FSubstrateSubsurfaceHeader SubstrateLoadSubsurfaceHeader(inout FSubstrateAddressing SubstrateAddressing)
{
	const FGBufferData GBufferData = GetScreenSpaceDataUint(SubstrateAddressing.PixelCoords).GBuffer;

	FSubstrateSubsurfaceHeader Out = (FSubstrateSubsurfaceHeader)0;

	uint Mode = SSS_TYPE_WRAP;
	if (GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE || GBufferData.ShadingModelID == SHADINGMODELID_PREINTEGRATED_SKIN)
	{
		Mode = SSS_TYPE_WRAP;
		SubstrateSubSurfaceHeaderSetWrapOpacity(Out, GBufferData.CustomData.a);
	}
	else if (GBufferData.ShadingModelID == SHADINGMODELID_TWOSIDED_FOLIAGE)
	{
		Mode = SSS_TYPE_TWO_SIDED_WRAP;
		SubstrateSubSurfaceHeaderSetWrapOpacity(Out, GBufferData.CustomData.a);

	}
	else if (GBufferData.ShadingModelID == SHADINGMODELID_SUBSURFACE_PROFILE ||
		GBufferData.ShadingModelID == SHADINGMODELID_EYE)
	{
		Mode = SSS_TYPE_DIFFUSION_PROFILE;
		SubstrateSubSurfaceHeaderSetProfile(Out, GBufferData.CustomData.a, ExtractSubsurfaceProfileInt(GBufferData));
	}
	else if (GBufferData.ShadingModelID == SHADINGMODELID_SUBSTRATE_TOON)
	{
		Mode = SSS_TYPE_TRANSMITTANCE_SHADOW;

		float2 PatternUVs = 0.0f;
		float ToonProfileId = 0.0f;
		UnpackToonCustomData(PackRGBA8(GBufferData.CustomData), PatternUVs, ToonProfileId);

		SubstrateSubSurfaceHeaderSetTransmittanceShadowMFPInMeters(Out, GetToonProfileShadowSettings(ToonProfileId).ShadowMFPInMeters.xxx);
	}

	SubstrateSubSurfaceHeaderSetSSSType(Out, Mode);
	return Out;
}

FSubstrateTopLayerData SubstrateUnpackTopLayerData(inout FSubstrateAddressing SubstrateAddressing)
{
	const FGBufferData GBufferData = GetScreenSpaceDataUint(SubstrateAddressing.PixelCoords).GBuffer;

	FSubstrateTopLayerData Out = (FSubstrateTopLayerData)0;
	if (GBufferData.ShadingModelID != SHADINGMODELID_UNLIT)
	{
		Out.WorldNormal = GBufferData.WorldNormal;
		Out.Roughness = GBufferData.Roughness;
		Out.Material = 1;
	}
	return Out;
}

#endif


#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
FSubstratePixelDebugData ConvertToSerializableSubstratePixelDebugData(uint2 InCoord, uint GPUFrameNumber)
#else
FSubstratePixelDebugData ConvertToSerializableSubstratePixelDebugData(uint2 InCoord, FSubstrateRaytracingPayload Payload, uint GPUFrameNumber)
#endif
{
	FSubstratePixelDebugData Data = (FSubstratePixelDebugData)0;

	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(InCoord, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	const uint FootPrint_Start = SubstrateAddressing.ReadBytes;

#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE && SUBSTRATE_GBUFFER_FORMAT == 1
	FSubstratePixelHeader Header = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
#elif SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE && SUBSTRATE_GBUFFER_FORMAT == 0
	FSubstratePixelHeader Header = UnpackSubstrateHeaderIn(SubstrateAddressing);
#else
	FSubstratePixelHeader Header = UnpackSubstrateHeaderIn(Payload, SubstrateAddressing, Payload);
#endif
	const uint FootPrint_PostHeader = SubstrateAddressing.ReadBytes;

#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE && SUBSTRATE_GBUFFER_FORMAT == 1
	FSubstrateSubsurfaceHeader SSSHeader = SubstrateLoadSubsurfaceHeader(Substrate.MaterialTextureArray, Substrate.FirstSliceStoringSubstrateSSSData, SubstrateAddressing.PixelCoords);
	FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Substrate.TopLayerTexture.Load(uint3(InCoord, 0)));
#elif SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE && SUBSTRATE_GBUFFER_FORMAT == 0
	FSubstrateSubsurfaceHeader SSSHeader = SubstrateLoadSubsurfaceHeader(SubstrateAddressing);
	FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(SubstrateAddressing);
#else
	FSubstrateSubsurfaceHeader SSSHeader = (FSubstrateSubsurfaceHeader)0;
	FSubstrateTopLayerData TopLayerData = SubstrateUnpackTopLayerData(Payload.PackedTopLayerData);
#endif
	Data.TopLayerRoughness = TopLayerData.Roughness;

	const bool bSubstrateMaterial = Header.ClosureCount > 0;
	const bool bIsSimpleMaterial = Header.IsSimpleMaterial() || Header.ClosureCount == 0;
	const bool bIsSingleMaterial = !Header.IsSimpleMaterial() && Header.IsSingleMaterial();

	FSubstrateIrradianceAndOcclusion IrradianceAndOcclusion = SubstrateGetIrradianceAndAO(Header);

	const uint ClampedClosureCount = clamp(Header.ClosureCount, 0, SUBSTRATE_VIS_CLOSURE_COUNT);
	Data.ClosureCount = ClampedClosureCount;

	Data.MaterialMode = Header.GetMaterialMode();
	Data.OptimisedLegacyMode = HEADER_MATERIALMODE_NONE;
	if (bIsSingleMaterial)
	{
#if SUBSTRATE_DEFERRED_SHADING
		Data.OptimisedLegacyMode = (Header.PackedHeader >> (HEADER_SINGLEENCODING_BIT_COUNT)) & HEADER_SINGLE_OPTLEGACYMODE_BIT_MASK;
#endif
	}
	Data.bIsComplexSpecialMaterial = Header.IsComplexSpecialMaterial();

	Data.MaterialAO = IrradianceAndOcclusion.MaterialAO;
	Data.IndirectIrradiance = IrradianceAndOcclusion.IndirectIrradiance;
	Data.HasPrecShadowMask = Header.HasPrecShadowMask() ? 1 : 0;
	Data.HasZeroPrecShadowMask = Header.HasZeroPrecShadowMask() ? 1 : 0;
	Data.DoesCastContactShadow = Header.DoesCastContactShadow() ? 1 : 0;
	Data.HasDynamicIndirectShadowCasterRepresentation = Header.HasDynamicIndirectShadowCasterRepresentation() ? 1 : 0;
	Data.HasSubsurface = Header.HasSubsurface() ? 1 : 0;
#if SUBSTRATE_DEFERRED_SHADING
	Data.LocalBasesCount = Header.GetLocalBasesCount();
#endif

	// PSSL - Temporarely limit data serialization on PSSL due to internal shader compilation 
	#if COMPILER_PSSL
	SUBSTRATE_UNROLL_N(SUBSTRATE_VIS_CLOSURE_COUNT)
	for(uint i = 0; i < ClampedClosureCount; ++i)
	#else
	LOOP
	for(uint i = 0; i < ClampedClosureCount; ++i)
	#endif
	{
		// Unpack BSDF data
	#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE && SUBSTRATE_GBUFFER_FORMAT == 1
		FSubstrateBSDF BSDF = UnpackSubstrateBSDF(Substrate.MaterialTextureArray, SubstrateAddressing, Header);
	#elif SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE && SUBSTRATE_GBUFFER_FORMAT == 0
		FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(SubstrateAddressing);
	#else
		FSubstrateBSDF BSDF = UnpackSubstrateBSDFIn(Payload, SubstrateAddressing, Header);
	#endif

		Data.Closures[i].Address = SubstrateAddressing.CurrentIndex;
		Data.Closures[i].Type = BSDF_GETTYPE(BSDF);

#if SUBSTRATE_DEFERRED_SHADING
		Data.Closures[i].NormalID = BSDF_GETSHAREDLOCALBASISID(BSDF);
		Data.Closures[i].BasisType = SubstrateGetBSDFBasisType(Header, BSDF);
#endif

		Data.Closures[i].bIsTopLayer = BSDF_GETISTOPLAYER(BSDF);
		Data.Closures[i].bHasWeightL = BSDF_GETHASTRANSABOVE(BSDF);
		Data.Closures[i].bHasGreyWeightV = BSDF_GETHASGREYWEIGHT_V(BSDF);

		Data.Closures[i].LuminanceWeightR = BSDF.LuminanceWeightV.r;
		Data.Closures[i].LuminanceWeightG = BSDF.LuminanceWeightV.g;
		Data.Closures[i].LuminanceWeightB = BSDF.LuminanceWeightV.b;
		Data.Closures[i].TransmittanceAboveAlongNR = BSDF.TransmittanceAboveAlongN.r;
		Data.Closures[i].TransmittanceAboveAlongNG = BSDF.TransmittanceAboveAlongN.g;
		Data.Closures[i].TransmittanceAboveAlongNB = BSDF.TransmittanceAboveAlongN.b;
		Data.Closures[i].CoverageAboveAlongN = BSDF.CoverageAboveAlongN;

		Data.Closures[i].SSSType = BSDF_GETSSSTYPE(BSDF);
		Data.Closures[i].bIsThin = BSDF_GETISTHIN(BSDF);

		switch (Data.Closures[i].Type)
		{
		case SUBSTRATE_BSDF_TYPE_SLAB:
		{
			Data.Closures[i].DiffuseR = SLAB_DIFFUSEALBEDO(BSDF).r;
			Data.Closures[i].DiffuseG = SLAB_DIFFUSEALBEDO(BSDF).g;
			Data.Closures[i].DiffuseB = SLAB_DIFFUSEALBEDO(BSDF).b;
			Data.Closures[i].F0R = SLAB_F0(BSDF).r;
			Data.Closures[i].F0G = SLAB_F0(BSDF).g;
			Data.Closures[i].F0B = SLAB_F0(BSDF).b;
			Data.Closures[i].Roughness = SLAB_ROUGHNESS(BSDF);

			Data.Closures[i].bHasF90 = BSDF_GETHASF90(BSDF);
			if (Data.Closures[i].bHasF90)
			{
				Data.Closures[i].F90R = SLAB_F90(BSDF).r;
				Data.Closures[i].F90G = SLAB_F90(BSDF).g;
				Data.Closures[i].F90B = SLAB_F90(BSDF).b;
			}
			else
			{
				Data.Closures[i].F90R = 1.0f;
				Data.Closures[i].F90G = 1.0f;
				Data.Closures[i].F90B = 1.0f;
			}

			Data.Closures[i].bHasAnisotropy = BSDF_GETHASANISOTROPY(BSDF);
			if (Data.Closures[i].bHasAnisotropy)
			{
				Data.Closures[i].Anisotropy = SLAB_ANISOTROPY(BSDF);
			}

			Data.Closures[i].bHasHaziness = BSDF_GETHASHAZINESS(BSDF);
			if (Data.Closures[i].bHasHaziness)
			{
				FHaziness Haziness = UnpackHaziness(SLAB_HAZINESS(BSDF));
				Data.Closures[i].HazeRoughness = Haziness.Roughness;
				Data.Closures[i].HazeWeight = Haziness.Weight;
				Data.Closures[i].HazeSimpleClearCoatMode = Haziness.bSimpleClearCoat ? 1u : 0u;
				Data.Closures[i].HasBottomNormal = Haziness.HasBottomNormal() ? 1u : 0u;
			}

			if (Data.Closures[i].SSSType != SSS_TYPE_NONE || Data.Closures[i].bIsThin)
			{
				if (Data.Closures[i].SSSType == SSS_TYPE_WRAP || Data.Closures[i].SSSType == SSS_TYPE_TWO_SIDED_WRAP)
				{
				#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
					Data.Closures[i].SSSOpacity = SubstrateSubSurfaceHeaderGetWrapOpacity(SSSHeader);
				#endif
					Data.Closures[i].SSSMFPR = SLAB_SSSMFP(BSDF).r;
					Data.Closures[i].SSSMFPG = SLAB_SSSMFP(BSDF).g;
					Data.Closures[i].SSSMFPB = SLAB_SSSMFP(BSDF).b;
					Data.Closures[i].SSSPhase = SLAB_SSSPHASEANISOTROPY(BSDF);
					Data.Closures[i].SSSThickness = BSDF_GETTHICKNESSCM(BSDF);
				}
				else if (Data.Closures[i].SSSType == SSS_TYPE_SIMPLEVOLUME)
				{
				#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
					Data.Closures[i].SSSOpacity = SubstrateSubSurfaceHeaderGetWrapOpacity(SSSHeader);
				#endif
					Data.Closures[i].SSSMFPR = SLAB_SSSMFP(BSDF).r;
					Data.Closures[i].SSSMFPG = SLAB_SSSMFP(BSDF).g;
					Data.Closures[i].SSSMFPB = SLAB_SSSMFP(BSDF).b;
					Data.Closures[i].SSSPhase = SLAB_SSSPHASEANISOTROPY(BSDF);
					Data.Closures[i].SSSThickness = BSDF_GETTHICKNESSCM(BSDF);
				}
				else if (Data.Closures[i].SSSType == SSS_TYPE_DIFFUSION_PROFILE)
				{
				#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
					const uint ProfileId = SubstrateSubSurfaceHeaderGetProfileId(SSSHeader);
					const float RadiusScale = SubstrateSubSurfaceHeaderGetProfileRadiusScale(SSSHeader);
					const float3 DiffuseMFP = GetSubsurfaceProfileMFPInCm(ProfileId).xyz * RadiusScale;
					const float Phase = GetTransmissionProfileParams(ProfileId).ScatteringDistribution;

					Data.Closures[i].SSSPRofileID = ProfileId;
					Data.Closures[i].SSSProfileRadius = RadiusScale;
					Data.Closures[i].SSSMFPR = DiffuseMFP.r;
					Data.Closures[i].SSSMFPG = DiffuseMFP.g;
					Data.Closures[i].SSSMFPB = DiffuseMFP.b;
					Data.Closures[i].SSSPhase = Phase;
				#endif
					if (Data.Closures[i].bIsThin)
					{
						Data.Closures[i].SSSThickness = SLAB_SSSPROFILETHICKNESSCM(BSDF);
					}
				}
				else if (Data.Closures[i].SSSType == SSS_TYPE_DIFFUSION)
				{
				#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
					const float3 OriginalMFP = SubstrateSubSurfaceHeaderGetMFP(SSSHeader);
					const float3 RescaledMFP = SLAB_SSSMFP(BSDF);


					Data.Closures[i].SSSMFPR = OriginalMFP.r;
					Data.Closures[i].SSSMFPG = OriginalMFP.g;
					Data.Closures[i].SSSMFPB = OriginalMFP.b;

					if (Data.Closures[i].bIsThin)
					{
						Data.Closures[i].SSSRescaledMFPR = RescaledMFP.r;
						Data.Closures[i].SSSRescaledMFPG = RescaledMFP.g;
						Data.Closures[i].SSSRescaledMFPB = RescaledMFP.b;
						Data.Closures[i].SSSThickness = BSDF_GETTHICKNESSCM(BSDF);
					}
				#endif

					Data.Closures[i].SSSPhase = SLAB_SSSPHASEANISOTROPY(BSDF);
				}
			}

			if (BSDF_GETHASFUZZ(BSDF))
			{
				Data.Closures[i].FuzzAmount = SLAB_FUZZ_AMOUNT(BSDF);
				Data.Closures[i].FuzzColorR = SLAB_FUZZ_COLOR(BSDF).r;
				Data.Closures[i].FuzzColorG = SLAB_FUZZ_COLOR(BSDF).g;
				Data.Closures[i].FuzzColorB = SLAB_FUZZ_COLOR(BSDF).b;
				Data.Closures[i].FuzzRoughness = SLAB_FUZZ_ROUGHNESS(BSDF);
			}

			if (BSDF_GETHASGLINT(BSDF))
			{
				Data.Closures[i].GlintValue = SLAB_GLINT_VALUE(BSDF);
				Data.Closures[i].GlintUVDDXx = SLAB_GLINT_UVDDX(BSDF).x;
				Data.Closures[i].GlintUVDDXy = SLAB_GLINT_UVDDX(BSDF).y;
				Data.Closures[i].GlintUVDDYx = SLAB_GLINT_UVDDY(BSDF).x;
				Data.Closures[i].GlintUVDDYy = SLAB_GLINT_UVDDY(BSDF).y;
			}
			else
			{
				Data.Closures[i].GlintValue = 1.0;	// No glint
			}

			Data.Closures[i].SpecProfileID = -1;
			if (BSDF_GETHASSPECPROFILE(BSDF))
			{
				Data.Closures[i].SpecProfileID = GetSpecularProfileId(SLAB_SPECPROFILEID(BSDF));
				Data.Closures[i].SpecProfileParameterization = GetSpecularProfileParameterization(SLAB_SPECPROFILEID(BSDF));
			}
		}
		break;

		case SUBSTRATE_BSDF_TYPE_HAIR:
		{
			// Reusing the slab data to avoid using too many registers in case the compiler has trouble optimising
			Data.Closures[i].DiffuseR	= HAIR_BASECOLOR(BSDF).r;
			Data.Closures[i].DiffuseG	= HAIR_BASECOLOR(BSDF).g;
			Data.Closures[i].DiffuseB	= HAIR_BASECOLOR(BSDF).b;
			Data.Closures[i].F0R		= HAIR_SPECULAR(BSDF);
			Data.Closures[i].Roughness	= HAIR_ROUGHNESS(BSDF);
			Data.Closures[i].F90R		= HAIR_SCATTER(BSDF);
			Data.Closures[i].F90G		= HAIR_BACKLIT(BSDF);
			Data.Closures[i].F90B		= HAIR_COMPLEXTRANSMITTANCE(BSDF);
		}
		break;

		case SUBSTRATE_BSDF_TYPE_EYE:
		{
			// Reusing the slab data to avoid using too many registers in case the compiler has trouble optimising
			Data.Closures[i].DiffuseR	= EYE_DIFFUSEALBEDO(BSDF).r;
			Data.Closures[i].DiffuseG	= EYE_DIFFUSEALBEDO(BSDF).g;
			Data.Closures[i].DiffuseB	= EYE_DIFFUSEALBEDO(BSDF).b;
			Data.Closures[i].F0R		= EYE_F0(BSDF).r;
			Data.Closures[i].Roughness	= EYE_ROUGHNESS(BSDF);
			Data.Closures[i].F90R		= EYE_IRISMASK(BSDF);
			Data.Closures[i].F90G		= EYE_IRISDISTANCE(BSDF);

			Data.Closures[i].SSSMFPR	= EYE_IRISNORMAL(BSDF).x;
			Data.Closures[i].SSSMFPG	= EYE_IRISNORMAL(BSDF).y;
			Data.Closures[i].SSSMFPB	= EYE_IRISNORMAL(BSDF).z;
			Data.Closures[i].SSSRescaledMFPR = EYE_IRISPLANENORMAL(BSDF).x;
			Data.Closures[i].SSSRescaledMFPG = EYE_IRISPLANENORMAL(BSDF).y;
			Data.Closures[i].SSSRescaledMFPB = EYE_IRISPLANENORMAL(BSDF).z;
			
		#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
			const bool bHasSSS = Header.HasSubsurface();
			const bool bIsValid = SubstrateSubSurfaceHeaderGetIsValid(SSSHeader);
			const bool bIsProfile = SubstrateSubSurfaceHeaderGetIsProfile(SSSHeader);
			Data.Closures[i].SSSPRofileID = -1;
			if (bHasSSS && bIsValid && bIsProfile)
			{
				Data.Closures[i].SSSPRofileID = SubstrateSubSurfaceHeaderGetProfileId(SSSHeader);
			}
		#endif
		}
		break;

		case SUBSTRATE_BSDF_TYPE_SINGLELAYERWATER:
		{
			// Reusing the slab data to avoid using too many registers in case the compiler has trouble optimising
			Data.Closures[i].DiffuseR	= SLW_BASECOLOR(BSDF).r;
			Data.Closures[i].DiffuseG	= SLW_BASECOLOR(BSDF).g;
			Data.Closures[i].DiffuseB	= SLW_BASECOLOR(BSDF).b;
			Data.Closures[i].F0R		= SLW_SPECULAR(BSDF).r;
			Data.Closures[i].F0G		= SLW_METALLIC(BSDF).r;
			Data.Closures[i].Roughness	= SLW_ROUGHNESS(BSDF);
			Data.Closures[i].SSSOpacity = SLW_TOPMATERIALOPACITY(BSDF);
		}
		break;

		case SUBSTRATE_BSDF_TYPE_TOON:
		{
			const float3 DiffuseColor = ComputeDiffuseAlbedo(TOON_BASECOLOR(BSDF), TOON_METALLIC(BSDF));
			const float3 SpecularColor = ComputeF0(TOON_SPECULAR(BSDF), TOON_BASECOLOR(BSDF), TOON_METALLIC(BSDF));

			Data.Closures[i].DiffuseR		= DiffuseColor.r;
			Data.Closures[i].DiffuseG		= DiffuseColor.g;
			Data.Closures[i].DiffuseB		= DiffuseColor.b;
			Data.Closures[i].F0R			= SpecularColor.r;
			Data.Closures[i].F0G			= SpecularColor.g;
			Data.Closures[i].F0B			= SpecularColor.b;
			Data.Closures[i].Roughness		= TOON_ROUGHNESS(BSDF);
			Data.Closures[i].SSSPRofileID	= TOON_PROFILEID(BSDF);

			Data.Closures[i].bHasAnisotropy = BSDF_GETHASANISOTROPY(BSDF);
			if (Data.Closures[i].bHasAnisotropy)
			{
				Data.Closures[i].Anisotropy = TOON_ANISOTROPY(BSDF);
			}
			
			Data.Closures[i].GlintUVDDXx	= TOON_PATTERNUVS(BSDF).x;
			Data.Closures[i].GlintUVDDXy	= TOON_PATTERNUVS(BSDF).y;

			if (Data.Closures[i].SSSType == SSS_TYPE_TRANSMITTANCE_SHADOW)
			{
			#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
				const float ShadowMFPInMeters = GetToonProfileShadowSettings(TOON_PROFILEID(BSDF)).ShadowMFPInMeters;
				Data.Closures[i].SSSMFPR = ShadowMFPInMeters;
				Data.Closures[i].SSSMFPG = ShadowMFPInMeters;
				Data.Closures[i].SSSMFPB = ShadowMFPInMeters;
			#endif
			}
		}
		break;

		default:
		{
			// Error
		}
		break;
		}
	}

	const uint FootPrint_PostBSDFs = SUBSTRATE_GBUFFER_FORMAT == 1 ? SubstrateAddressing.ReadBytes : 16u; // 4 * 32bits MRTs

	// Output memory reads
	Data.MemoryDisplayMode = 0;
	{
		const bool bHasSSSData = SUBSTRATE_GBUFFER_FORMAT == 1 && SubstrateSubSurfaceHeaderGetIsValid(SSSHeader);
		const uint TopLayerDataBytes = SUBSTRATE_GBUFFER_FORMAT == 1 ? 4 : 0;
		const uint SSSDataBytes = 8;

		const uint HeaderSize = FootPrint_PostHeader - FootPrint_Start;
		const uint BSDFsSize  = FootPrint_PostBSDFs  - FootPrint_PostHeader;
		const uint TotalSize  = (FootPrint_PostBSDFs - FootPrint_Start) + (bHasSSSData ? SSSDataBytes : 0);

		if (ClampedClosureCount > 0)
		{
			if (bIsSimpleMaterial || Header.IsSingleLayerWater())
			{
				Data.MemoryDisplayMode = 1;
				Data.MemorySlotA = HeaderSize + BSDFsSize - TopLayerDataBytes;	// Header+BSDF
				Data.MemorySlotB = TopLayerDataBytes;							// TopNormalTex
			}
			else if (bIsSingleMaterial || Header.IsEye() || Header.IsHair())
			{
				Data.MemoryDisplayMode = 2;
				Data.MemorySlotA = HeaderSize - TopLayerDataBytes;				// Header
				Data.MemorySlotB = TopLayerDataBytes;							// TopNormalTex
				Data.MemorySlotC = BSDFsSize;									// BSDF
			}
			else
			{
				Data.MemoryDisplayMode = 3;
				Data.MemorySlotA = HeaderSize;									// Header+Norm
				Data.MemorySlotB = BSDFsSize;									// BSDFs
			}

			if (bHasSSSData)
			{
				Data.MemorySSSData = SSSDataBytes;								// SSS Data
			}
		}

		Data.MemoryTotal = TotalSize;
	}

	Data.GPUFrameNumber = asint(GPUFrameNumber);

	return Data;
}

#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
void SerializeSubstratePixelDebugDataEntry(uint2 InCoord, uint GPUFrameNumber)
#else
void SerializeSubstratePixelDebugDataEntry(uint2 InCoord, FSubstrateRaytracingPayload Payload, uint GPUFrameNumber)
#endif
{
	// Convert Substrate GBuffer into the CPU/GPU common data to serialize.
#if SUBSTRATE_MATERIALCONTAINER_IS_VIEWRESOURCE
	FSubstratePixelDebugData Data = ConvertToSerializableSubstratePixelDebugData(InCoord, GPUFrameNumber);
#else
	FSubstratePixelDebugData Data = ConvertToSerializableSubstratePixelDebugData(InCoord, Payload, GPUFrameNumber);
#endif
	
	// And serialise it.
	FSubstrateDebugDataSerializer S = (FSubstrateDebugDataSerializer)0;
	SerializeSubstratePixelDebugData(S, Data);
}

void SerializeNullPixelDebugDataEntry()
{
	FSubstratePixelDebugData Data = (FSubstratePixelDebugData)0;
	FSubstrateDebugDataSerializer S = (FSubstrateDebugDataSerializer)0;
	SerializeSubstratePixelDebugData(S, Data);
}

#endif


////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////


struct FSubstrateSystemInfoData
{
	int TileCount[SUBSTRATE_TILE_TYPE_COUNT];
};

#ifndef __cplusplus
void SerializeSubstrateSystemInfoDebugData(in FSubstrateDebugDataSerializer S, in FSubstrateSystemInfoData D)
#else
void SerializeSubstrateSystemInfoDebugData(FSubstrateDebugDataSerializer& S, FSubstrateSystemInfoData& D)
#endif
{
	for (int i = 0; i < SUBSTRATE_TILE_TYPE_COUNT; ++i)
	{
		S.Serialize(D.TileCount[i]);
	}
}

#ifndef __cplusplus

uint GetTileCount(uint InType);

void ConvertToSerializableSubstratePixelDebugData()
{
	FSubstrateSystemInfoData Data = (FSubstrateSystemInfoData)0;

	// Setup the system info to send to the CPU for debug print
	for (int i = 0; i < SUBSTRATE_TILE_TYPE_COUNT; ++i)
	{
		Data.TileCount[i] = GetTileCount(i);
	}

	// And serialise it.
	FSubstrateDebugDataSerializer S = (FSubstrateDebugDataSerializer)0;
	SerializeSubstrateSystemInfoDebugData(S, Data);
}

#endif

#endif // defined(__cplusplus) || SUBSTRATE_ENABLED
