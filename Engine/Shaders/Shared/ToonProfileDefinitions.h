// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define TOON_PROFILE_DIFFUSE_RAMP_Y		0
#define TOON_PROFILE_SPECULAR_RAMP_Y	1
#define TOON_PROFILE_SHADOW_RAMP_Y		2
#define TOON_PROFILE_SPEC_IND_RAMP_Y	3
#define TOON_PROFILE_PARAMETERS_Y		4
#define TOON_PROFILE_TEXTURES_Y			5

#define TOON_PROFILE_EVAL_PARAMS_X		0
#define TOON_PROFILE_EVAL_PARAMS2_X		1
#define TOON_PROFILE_SHADOW_PARAMS_X	2
#define TOON_PROFILE_GI_PARAMS_X		3

#define TOON_PROFILE_TEXTURE_RAMPOFFSET_IDX			0
#define TOON_PROFILE_TEXTURE_SHADOWHATCHING_IDX		1
#define TOON_PROFILE_TEXTURE_COUNT					2
