// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define VIEW_DATA_ACCESS_SCENE_UB 0
#define VIEW_DATA_ACCESS_RW 1
