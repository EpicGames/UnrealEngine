// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================================
VirtualShadowMapDefinitions.h: used in virtual shadow map shaders and C++ code to define common constants
!!! Changing this file requires recompilation of the engine !!!
=================================================================================================*/

#pragma once

#define VSM_ENABLE_VISUALIZATION (WITH_EDITOR || !UE_BUILD_SHIPPING)

// Page size is 128x128
#define VSM_LOG2_PAGE_SIZE 7u
#define VSM_PAGE_SIZE (1u << VSM_LOG2_PAGE_SIZE)
#define VSM_PAGE_SIZE_MASK (VSM_PAGE_SIZE - 1u)
// Page table size is 128x128 (total 16k)
#define VSM_LOG2_LEVEL0_DIM_PAGES_XY 7u
#define VSM_LEVEL0_DIM_PAGES_XY (1u << VSM_LOG2_LEVEL0_DIM_PAGES_XY)
#define VSM_MAX_MIP_LEVELS (VSM_LOG2_LEVEL0_DIM_PAGES_XY + 1u)
#define VSM_VIRTUAL_MAX_RESOLUTION_XY (VSM_LEVEL0_DIM_PAGES_XY * VSM_PAGE_SIZE)
#define VSM_RASTER_WINDOW_PAGES (4u)

// Number of per-page slices in Shadow.Virtual.DirtyPageFlags. 
#define VSM_DIRTY_PAGE_FLAGS_NUM_SLICES 6

// 1x border for bilinear filtering
#define VSM_PREFILTERED_DISTANT_BORDER_SIZE 1u
#define VSM_PREFILTERED_DISTANT_PAGE_SIZE (VSM_PAGE_SIZE + 2u * VSM_PREFILTERED_DISTANT_BORDER_SIZE)

// receiver mask is 8x8
#define VSM_LOG2_RECEIVER_MASK_SIZE 3u
#define VSM_RECEIVER_MASK_SIZE (1u << VSM_LOG2_RECEIVER_MASK_SIZE)
#define VSM_RECEIVER_MASK_MASK (VSM_RECEIVER_MASK_SIZE - 1u)
#define VSM_RECEIVER_MASK_SUBMASK (VSM_RECEIVER_MASK_MASK >> 1u)

// Page table layout in a 2D texture (array, probably or atlas perhaps) base level, with space for mips next to it
#define VSM_PAGE_TABLE_TEX2D_SIZE_X (VSM_LEVEL0_DIM_PAGES_XY)
#define VSM_PAGE_TABLE_TEX2D_SIZE_Y (VSM_LEVEL0_DIM_PAGES_XY + VSM_LEVEL0_DIM_PAGES_XY / 2)

#define VIRTUAL_SHADOW_MAP_VISUALIZE_NONE						0
#define VIRTUAL_SHADOW_MAP_VISUALIZE_SHADOW_FACTOR				(1 << 0)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_CLIPMAP_OR_MIP				(1 << 1)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_VIRTUAL_PAGE				(1 << 2)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_CACHED_PAGE				(1 << 3)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_SMRT_RAY_COUNT				(1 << 4)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_CLIPMAP_VIRTUAL_SPACE		(1 << 5)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_GENERAL_DEBUG				(1 << 6)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_DIRTY_PAGE					(1 << 7)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_GPU_INVALIDATED_PAGE		(1 << 8)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_MERGED_PAGE				(1 << 9)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_NANITE_OVERDRAW			(1 << 10)
#define VIRTUAL_SHADOW_MAP_VISUALIZE_SHADOW_CASTERS				(1 << 11)

// Bias used to store negative clip levels in less than 32 bits
#define VSM_PACKED_CLIP_LEVEL_BIAS	1024

#define VSM_PROJ_FLAG_CURRENT_DISTANT_LIGHT			(1U << 0)
#define VSM_PROJ_FLAG_UNCACHED						(1U << 1) // Used to indicate that the light is uncached and should only render to dynamic pages
#define VSM_PROJ_FLAG_UNREFERENCED					(1U << 2) // Used to indicate that the light is not referenced/rendered to this render
#define VSM_PROJ_FLAG_IS_COARSE_CLIP_LEVEL			(1U << 3) // Used to indicate that the clip level is a coarse level
#define VSM_PROJ_FLAG_IS_FIRST_PERSON_SHADOW		(1U << 4) // Used to indicate that this is a "first-person" shadow 
#define VSM_PROJ_FLAG_USE_RECEIVER_MASK				(1U << 5) // Used to enable receiver masks on this light
#define VSM_PROJ_FLAG_FORCE_CACHE_DYNAMIC_COARSE	(1U << 6) // Used to prevent invalidations of dynamic coarse pages, that is pages NOT marked with detail geometry.
#define VSM_PROJ_FLAG_HAS_PREFILTERED_DISTANT		(1U << 7) // Allocate and update a filterable shadow representation

// Hard limit for max distant lights supported 8k for now - we may revise later. We need to keep them in a fixed range for now to make allocation easy and minimize overhead for indexing.
#define VSM_MAX_SINGLE_PAGE_SHADOW_MAPS (1024U * 8U)

#define VSM_INVALIDATION_PAYLOAD_FLAG_NONE                      0
#define VSM_INVALIDATION_PAYLOAD_FLAG_FORCE_STATIC              (1 << 0)
// 8 bit flags, 24 bit VSM ID
#define VSM_INVALIDATION_PAYLOAD_FLAG_BITS                      8


#define VSM_STAT_REQUESTED_THIS_FRAME_PAGES				 0
#define VSM_STAT_STATIC_CACHED_PAGES					 1
#define VSM_STAT_STATIC_INVALIDATED_PAGES				 2
#define VSM_STAT_DYNAMIC_CACHED_PAGES					 3
#define VSM_STAT_DYNAMIC_INVALIDATED_PAGES				 4
#define VSM_STAT_EMPTY_PAGES							 5
#define VSM_STAT_NON_NANITE_INSTANCES_TOTAL				 6
#define VSM_STAT_NON_NANITE_INSTANCES_DRAWN				 7
#define VSM_STAT_NON_NANITE_INSTANCES_HZB_CULLED		 8
#define VSM_STAT_NON_NANITE_INSTANCES_PAGE_MASK_CULLED	 9
#define VSM_STAT_NON_NANITE_INSTANCES_EMPTY_RECT_CULLED	10
#define VSM_STAT_NON_NANITE_INSTANCES_FRUSTUM_CULLED	11
#define VSM_STAT_NUM_PAGES_TO_MERGE						12
#define VSM_STAT_NUM_PAGES_TO_CLEAR						13
#define VSM_STAT_NUM_HZB_PAGES_BUILT					14
#define VSM_STAT_ALLOCATED_NEW							15
#define VSM_STAT_NANITE_CLUSTERS_HW						16
#define VSM_STAT_NANITE_CLUSTERS_SW						17
#define VSM_STAT_NANITE_TRIANGLES						18
#define VSM_STAT_NANITE_INSTANCES_MAIN					19
#define VSM_STAT_NANITE_INSTANCES_POST					20
#define VSM_STAT_WPO_CONSIDERED_PAGES					21
#define VSM_STAT_OVERFLOW_FLAGS							22
#define VSM_STAT_PREFILTERED_DISTANT_PAGES				23
#define VSM_STAT_TMP_2									24
#define VSM_STAT_TMP_3									25
#define VSM_STAT_NUM									26

#define VSM_STAT_OVERFLOW_FLAG_MARKING_JOB_QUEUE 		(1<<0)
#define VSM_STAT_OVERFLOW_FLAG_OPP_MAX_LIGHTS	 		(1<<1)
#define VSM_STAT_OVERFLOW_FLAG_PAGE_POOL		 		(1<<2)
#define VSM_STAT_OVERFLOW_FLAG_VISIBLE_INSTANCES 		(1<<3)
#define VSM_STAT_OVERFLOW_FLAG_NUM						4

#define VSM_STATUS_MSG_PAGE_MANAGEMENT					0
#define VSM_STATUS_MSG_OVERFLOW							1

// Projection Shader Data
#define VSM_PSD_STRIDE (16 * 23)

// Nanite Performance Feedback
#define VSM_NPF_HEADER_TOTAL_HW_CLUSTERS				0
#define VSM_NPF_HEADER_TOTAL_SW_CLUSTERS				1
#define VSM_NPF_SIZEOF_HEADER							2
#define VSM_NPF_ENTRY_HW_CLUSTERS						0
#define VSM_NPF_ENTRY_SW_CLUSTERS						1
#define VSM_NPF_SIZEOF_ENTRY							2

// Throttle Buffer
#define VSM_TB_HEADER_TOTAL_HW_CLUSTERS				0
#define VSM_TB_HEADER_TOTAL_SW_CLUSTERS				1
#define VSM_TB_HEADER_MAX_HW_CLUSTERS				2
#define VSM_TB_HEADER_MAX_SW_CLUSTERS				3
#define VSM_TB_HEADER_TOTAL_THROTTLE				4
#define VSM_TB_SIZEOF_HEADER						5
#define VSM_TB_ENTRY_HW_CLUSTERS					0
#define VSM_TB_ENTRY_SW_CLUSTERS					1
#define VSM_TB_ENTRY_THROTTLE						2
#define VSM_TB_SIZEOF_ENTRY							3

// Next data
#define VSM_NEXT_FLAG_KEEP_PAGE	(1u << 0)
#define VSM_NEXT_FLAG_ID_VALID	(1u << 1)

#ifdef __cplusplus
#include "HLSLTypeAliases.h"
#include "HLSLMathAliases.h"

namespace UE::HLSL
{
#endif

struct FPhysicalPageMetaData
{
	uint Flags;	// VSM_FLAG_* (not all relevant to physical pages) and VSM_EXTENDED_FLAG_*
	uint LastRequestedSceneFrameNumber;
	// Link back to the virtual shadow map & page table slot that references this physical page
	uint VirtualShadowMapId;
	uint MipLevel;
	uint2 PageAddress;
};

struct FVSMVisibleInstanceCmd
{
	uint PackedPageInfo;
	uint InstanceIdAndFlags;
	uint IndirectArgIndex;
};

struct FVSMCullingBatchInfo
{
	uint FirstPrimaryView;
	uint NumPrimaryViews;
};

struct FNextVirtualShadowMapData
{
	uint Flags;
	int NextVirtualShadowMapId;
	int2 PageAddressOffset;
};

//ExtraBias must >= 0
inline uint GetMipLevelLocal(float Footprint, uint MipMode, float ShadowMapResolutionLodBias, float GlobalResolutionLodBias, float ExtraBias = 0.0f)
{
	float MipLevelFloat = log2(Footprint) + ShadowMapResolutionLodBias + GlobalResolutionLodBias + ExtraBias;
	uint MipLevel = uint(max(floor(MipLevelFloat), 0.0f));
	MipLevel = min(MipLevel, (VSM_MAX_MIP_LEVELS - 1U));

	if (MipMode == 1)
	{
		// Even mips
		MipLevel &= ~(1U);
	}
	else if (MipMode == 2)
	{
		// Odd mips
		MipLevel |= 1U;
	}

	return MipLevel;
}

struct EVSMStatSection
{
	enum Type
	{
		None = 0,
		Basic = 1U << 0,

		PhysicalPages = 1U << 1,
		NonNanite = 1U << 2,
		Clusters = 1U << 3,

		MAX = 1U << 4,

		All = (MAX - 1)
	};
};

#define ENUMERATE_VSM_STAT_SECTIONS(op) \
	op(PhysicalPages) \
	op(NonNanite) \
	op(Clusters) \

#ifdef __cplusplus
} // namespace UE::HLSL

using FVSMVisibleInstanceCmd = UE::HLSL::FVSMVisibleInstanceCmd;
using FVSMCullingBatchInfo = UE::HLSL::FVSMCullingBatchInfo;
using FNextVirtualShadowMapData = UE::HLSL::FNextVirtualShadowMapData;
using EVSMStatSection = UE::HLSL::EVSMStatSection;

#endif
