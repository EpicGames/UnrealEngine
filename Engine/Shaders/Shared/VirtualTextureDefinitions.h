// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifdef __cplusplus
#include "HLSLTypeAliases.h"
namespace UE::HLSL
{
#endif  // __cplusplus

/** Bias applied to all feedback, allows for negative (mag) levels */
static const int VirtualTextureFeedbackBias = 3;
	
#ifdef __cplusplus
} // namespace
#endif // __cplusplus
